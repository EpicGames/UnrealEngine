// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FloatingProperties : ModuleRules
{
	public FloatingProperties(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DeveloperSettings"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AppFramework",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"InputCore",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd"
			}
		);
	}
}
