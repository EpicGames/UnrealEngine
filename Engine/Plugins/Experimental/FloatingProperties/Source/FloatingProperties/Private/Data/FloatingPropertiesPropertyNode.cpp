// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/FloatingPropertiesPropertyNode.h"
#include "Data/FloatingPropertiesPropertyNodeContainer.h"
#include "FloatingPropertiesModule.h"
#include "FloatingPropertiesSettings.h"
#include "Math/Vector2D.h"
#include "Types/SlateEnums.h"
#include "Widgets/SFloatingPropertiesPropertyWidget.h"
#include "Widgets/SWidget.h"

EFloatingPropertiesUpdateResult UE::FloatingProperties::CombineUpdateResult(EFloatingPropertiesUpdateResult InCurrentResult, 
	EFloatingPropertiesUpdateResult InNewResult)
{
	if (InNewResult == EFloatingPropertiesUpdateResult::Failed)
	{
		return EFloatingPropertiesUpdateResult::Failed;
	}

	switch (InCurrentResult)
	{
		case EFloatingPropertiesUpdateResult::Failed:
			return EFloatingPropertiesUpdateResult::Failed;

		case EFloatingPropertiesUpdateResult::Updated:
			return EFloatingPropertiesUpdateResult::Updated;

		case EFloatingPropertiesUpdateResult::AlreadyUpToDate:
		default:
			return InNewResult;
	}
}

FFloatingPropertiesPropertyNode::FFloatingPropertiesPropertyNode(FloatingPropertiesPropertyNodeContainer* InLinkedList,
	TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget, int32 InIndex)
	: LinkedList(InLinkedList)
    , MyIndex(InIndex)
    , ParentNodeIndex(INDEX_NONE)
    , ChildNodeIndex(INDEX_NONE)
	, PropertyWidget(InPropertyWidget)
	, PropertyAnchor(FFloatingPropertiesClassPropertyAnchor())
	, PropertyPosition(FFloatingPropertiesClassPropertyPosition::DefaultStackPosition)
    , CachedPosition(FVector2f::ZeroVector)
    , bValidPosition(false)
{		
}

void FFloatingPropertiesPropertyNode::SetChild(TSharedRef<FFloatingPropertiesPropertyNode> InChild)
{
	if (InChild->MyIndex == MyIndex
		|| InChild->ParentNodeIndex == MyIndex
		|| InChild->ChildNodeIndex == MyIndex
		|| InChild->MyIndex == ParentNodeIndex
		|| InChild->MyIndex == ChildNodeIndex)
	{
		return;
	}

	if (TSharedPtr<FFloatingPropertiesPropertyNode> Child = GetChild())
	{
		if (Child->MyIndex == InChild->MyIndex)
		{
			return;
		}

		RemoveChild();
	}

	InChild->RemoveParent();	

	ChildNodeIndex = InChild->MyIndex;
	PropertyAnchor.ChildProperty = InChild->PropertyWidget->GetClassProperty();

	InChild->ParentNodeIndex = MyIndex;
	InChild->PropertyAnchor.ParentProperty = PropertyWidget->GetClassProperty();
	InChild->UpdateStackPositions(/* bInInvalidate */ true);
}

void FFloatingPropertiesPropertyNode::SetParent(TSharedPtr<FFloatingPropertiesPropertyNode> InParentNode)
{
	InParentNode->SetChild(SharedThis(this));
}

TSharedPtr<FFloatingPropertiesPropertyNode> FFloatingPropertiesPropertyNode::RemoveChild()
{
	TSharedPtr<FFloatingPropertiesPropertyNode> Child = GetChild();

	if (!Child.IsValid())
	{
		return nullptr;
	}

	ChildNodeIndex = INDEX_NONE;
	PropertyAnchor.ChildProperty.Reset();

	Child->ParentNodeIndex = INDEX_NONE;
	Child->PropertyAnchor.ParentProperty.Reset();

	Child->PropertyPosition = CalculatePropertyPosition(
		LinkedList->GetDraggableArea(), 
		Child->GetPropertyWidget()->GetAbsolutePropertySize(),
		Child->GetCachedPosition()
	);

	Child->UpdateStackPositions(/* bInInvalidate */ true);

	return Child;
}

TSharedPtr<FFloatingPropertiesPropertyNode> FFloatingPropertiesPropertyNode::RemoveParent()
{
	if (TSharedPtr<FFloatingPropertiesPropertyNode> Parent = GetParent())
	{
		Parent->RemoveChild();
		return Parent;
	}

	return nullptr;
}

void FFloatingPropertiesPropertyNode::InsertAfter(TSharedRef<FFloatingPropertiesPropertyNode> InMiddle)
{
	RemoveFromStack();

	TSharedPtr<FFloatingPropertiesPropertyNode> OldChild = InMiddle->RemoveChild();

	InMiddle->SetChild(SharedThis(this));

	if (OldChild.IsValid())
	{
		TSharedRef<FFloatingPropertiesPropertyNode> Leaf = GetStackLeafMostNode();
		Leaf->SetChild(OldChild.ToSharedRef());
	}	
}

void FFloatingPropertiesPropertyNode::RemoveFromStack()
{
	TSharedPtr<FFloatingPropertiesPropertyNode> Child = RemoveChild();
	TSharedPtr<FFloatingPropertiesPropertyNode> Parent = RemoveParent();

	if (Parent.IsValid() && Child.IsValid())
	{
		Parent->SetChild(Child.ToSharedRef());
	}
}

bool FFloatingPropertiesPropertyNode::HasParent() const
{
	return ParentNodeIndex != MyIndex && LinkedList->GetNodes().IsValidIndex(ParentNodeIndex);
}

bool FFloatingPropertiesPropertyNode::HasParent(TSharedRef<FFloatingPropertiesPropertyNode> InParent) const
{
	TSharedPtr<const FFloatingPropertiesPropertyNode> LoopNode = SharedThis(this);

	while (LoopNode.IsValid())
	{
		if (LoopNode == InParent)
		{
			return true;
		}

		LoopNode = LoopNode->GetParent();
	}

	return false;
}

TSharedPtr<FFloatingPropertiesPropertyNode> FFloatingPropertiesPropertyNode::GetParent() const
{
	// Error detection
	if (ParentNodeIndex == MyIndex)
	{
		UE_LOGF(LogFloatingProperties, Warning, "Invalid parent relationship detected in GetParent.");
		return nullptr;
	}

	if (LinkedList->GetNodes().IsValidIndex(ParentNodeIndex))
	{
		return LinkedList->GetNodes()[ParentNodeIndex];
	}

	return nullptr;
}

bool FFloatingPropertiesPropertyNode::HasChild() const
{
	return ChildNodeIndex != MyIndex && LinkedList->GetNodes().IsValidIndex(ChildNodeIndex);
}

bool FFloatingPropertiesPropertyNode::HasChild(TSharedRef<FFloatingPropertiesPropertyNode> InChild) const
{
	return InChild->HasParent(SharedThis(const_cast<FFloatingPropertiesPropertyNode*>(this)));
}

TSharedPtr<FFloatingPropertiesPropertyNode> FFloatingPropertiesPropertyNode::GetChild() const
{
	// Error detection
	if (ChildNodeIndex == MyIndex)
	{
		UE_LOGF(LogFloatingProperties, Warning, "Invalid parent relationship detected in GetChild.");
		return nullptr;
	}

	if (LinkedList->GetNodes().IsValidIndex(ChildNodeIndex))
	{
		return LinkedList->GetNodes()[ChildNodeIndex];
	}

	return nullptr;
}

bool FFloatingPropertiesPropertyNode::HasSibling(TSharedRef<FFloatingPropertiesPropertyNode> InSibling) const
{
	TSharedRef<FFloatingPropertiesPropertyNode> This = SharedThis(const_cast<FFloatingPropertiesPropertyNode*>(this));

	if (This == InSibling)
	{
		return true;
	}

	TSharedPtr<FFloatingPropertiesPropertyNode> LoopNode = This;

	while (TSharedPtr<FFloatingPropertiesPropertyNode> Parent = LoopNode->GetParent())
	{
		LoopNode = Parent.ToSharedRef();

		if (LoopNode == InSibling)
		{
			return true;
		}
	}

	LoopNode = This;

	while (TSharedPtr<FFloatingPropertiesPropertyNode> Child = LoopNode->GetChild())
	{
		LoopNode = Child.ToSharedRef();

		if (LoopNode == InSibling)
		{
			return true;
		}
	}

	return false;
}

TSharedRef<FFloatingPropertiesPropertyNode> FFloatingPropertiesPropertyNode::GetStackRootNode() const
{
	TSharedRef<FFloatingPropertiesPropertyNode> LoopNode = SharedThis(const_cast<FFloatingPropertiesPropertyNode*>(this));
	
	while (TSharedPtr<FFloatingPropertiesPropertyNode> Parent = LoopNode->GetParent())
	{
		LoopNode = Parent.ToSharedRef();
	}

	return LoopNode;
}

TSharedRef<FFloatingPropertiesPropertyNode> FFloatingPropertiesPropertyNode::GetStackLeafMostNode() const
{
	TSharedRef<FFloatingPropertiesPropertyNode> LoopNode = SharedThis(const_cast<FFloatingPropertiesPropertyNode*>(this));

	while (TSharedPtr<FFloatingPropertiesPropertyNode> Child = LoopNode->GetChild())
	{
		LoopNode = Child.ToSharedRef();
	}

	return LoopNode;
}

const FFloatingPropertiesClassPropertyAnchor& FFloatingPropertiesPropertyNode::GetPropertyAnchor() const
{
	return PropertyAnchor;
}

const FFloatingPropertiesClassPropertyPosition& FFloatingPropertiesPropertyNode::GetPropertyPosition() const
{
	return PropertyPosition;
}

FFloatingPropertiesClassPropertyPosition FFloatingPropertiesPropertyNode::CalculatePropertyPosition(const FVector2f& InDraggableArea, 
	const FVector2f& InStackSize, const FVector2f& InPosition)
{
	FFloatingPropertiesClassPropertyPosition OutPosition = FFloatingPropertiesClassPropertyPosition::DefaultStackPosition;

	constexpr float OneThird = 1.f / 3.f;
	constexpr float TwoThirds = 2.f / 3.f;

	const FVector2f TopLeft = InDraggableArea * OneThird;
	const FVector2f BottomRight = InDraggableArea * TwoThirds;

	if (InPosition.X < TopLeft.X)
	{
		OutPosition.HorizontalAnchor = EHorizontalAlignment::HAlign_Left;
	}
	else if (InPosition.X >= BottomRight.X)
	{
		OutPosition.HorizontalAnchor = EHorizontalAlignment::HAlign_Right;
	}
	else
	{
		OutPosition.HorizontalAnchor = EHorizontalAlignment::HAlign_Center;
	}

	if (InPosition.Y < TopLeft.Y)
	{
		OutPosition.VerticalAnchor = EVerticalAlignment::VAlign_Top;
	}
	else if (InPosition.Y >= BottomRight.Y)
	{
		OutPosition.VerticalAnchor = EVerticalAlignment::VAlign_Bottom;
	}
	else
	{
		OutPosition.VerticalAnchor = EVerticalAlignment::VAlign_Center;
	}

	const FVector2f AnchorMultiplier = CalculateAnchorMultiplier(OutPosition);
	const FVector2f AvailableDragArea = InDraggableArea - InStackSize;
	const FVector2f AnchorOffset = InPosition - (AnchorMultiplier * AvailableDragArea);

	OutPosition.Offset.X = FMath::RoundToInt(AnchorOffset.X);
	OutPosition.Offset.Y = FMath::RoundToInt(AnchorOffset.Y);

	return OutPosition;
}

TSharedRef<SFloatingPropertiesPropertyWidget> FFloatingPropertiesPropertyNode::GetPropertyWidget() const
{
	return PropertyWidget;
}

EFloatingPropertiesUpdateResult FFloatingPropertiesPropertyNode::UpdateStackPositions(bool bInInvalidate)
{
	EFloatingPropertiesUpdateResult Result = EFloatingPropertiesUpdateResult::AlreadyUpToDate;
	TSharedPtr<FFloatingPropertiesPropertyNode> LoopNode = SharedThis(this);

	using namespace UE::FloatingProperties;

	while (LoopNode.IsValid())
	{
		if (bInInvalidate)
		{
			LoopNode->bValidPosition = false;
		}

		Result = CombineUpdateResult(Result, LoopNode->UpdatePropertyNodePosition());

		if (Result == EFloatingPropertiesUpdateResult::Failed)
		{
			return EFloatingPropertiesUpdateResult::Failed;
		}

		LoopNode = LoopNode->GetChild();
	}

	return Result;
}

void FFloatingPropertiesPropertyNode::InvalidateCachedPosition()
{
	bValidPosition = false;
}

const FVector2f& FFloatingPropertiesPropertyNode::GetCachedPosition() const
{
	return CachedPosition;
}

void FFloatingPropertiesPropertyNode::SetPropertyPositionDirect(const FFloatingPropertiesClassPropertyPosition& InPropertyPosition)
{
	PropertyPosition = InPropertyPosition;
}

void FFloatingPropertiesPropertyNode::SetCachedPosition(const FVector2f& InPosition)
{
	CachedPosition = InPosition;
	bValidPosition = true;
}

void FFloatingPropertiesPropertyNode::SetPropertyPosition(const FVector2f& InDraggableArea, const FVector2f& InPosition)
{
	PropertyPosition = CalculatePropertyPosition(InDraggableArea, GetStackSize(), InPosition);

	UpdateStackPositions(/* bInInvalidate */ true);
}

TArray<TSharedRef<FFloatingPropertiesPropertyNode>> FFloatingPropertiesPropertyNode::GetNodeStack() const
{
	TArray<TSharedRef<FFloatingPropertiesPropertyNode>> NodeStack;
	NodeStack.Reserve(LinkedList->GetNodes().Num());

	TSharedPtr<FFloatingPropertiesPropertyNode> LoopNode = GetStackRootNode();

	while (LoopNode.IsValid())
	{
		NodeStack.Add(LoopNode.ToSharedRef());
		LoopNode = LoopNode->GetChild();
	}

	return NodeStack;
}

FVector2f FFloatingPropertiesPropertyNode::GetStackSize() const
{
	FVector2f StackSize = FVector2f::ZeroVector;

	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : GetNodeStack())
	{
		const FVector2f NodeWidgetSize = Node->GetPropertyWidget()->GetAbsolutePropertySize();
		StackSize.X = FMath::Max(StackSize.X, NodeWidgetSize.X);
		StackSize.Y += NodeWidgetSize.Y;
	}

	return StackSize;
}

EFloatingPropertiesUpdateResult FFloatingPropertiesPropertyNode::UpdatePropertyNodePosition()
{
	if (bValidPosition)
	{
		return EFloatingPropertiesUpdateResult::AlreadyUpToDate;
	}

	const FVector2f DraggableArea = LinkedList->GetDraggableArea();
	constexpr float MinAxisValue = UE_SMALL_NUMBER;

	if (DraggableArea.X < MinAxisValue || DraggableArea.Y < MinAxisValue)
	{
		return EFloatingPropertiesUpdateResult::Failed;
	}

	UFloatingPropertiesSettings* FloatingPropertiesSettings = GetMutableDefault<UFloatingPropertiesSettings>();
	const FFloatingPropertiesClassProperty ClassProperty = PropertyWidget->GetClassProperty();

	// Has a parent, so attach it to the parent's position
	if (TSharedPtr<FFloatingPropertiesPropertyNode> ParentNode = GetParent())
	{
		if (ParentNode->MyIndex != MyIndex)
		{
			ParentNode->UpdatePropertyNodePosition();
		}

		const FVector2f& PreviousNodePosition = ParentNode->CachedPosition;
		const FVector2f PreviousNodeSize = ParentNode->PropertyWidget->GetAbsolutePropertySize();

		SetCachedPosition({
			PreviousNodePosition.X,
			PreviousNodePosition.Y + PreviousNodeSize.Y
		});

		if (GetStackRootNode()->GetPropertyPosition().OnDefaultStack())
		{
			FloatingPropertiesSettings->PropertyAnchors.Remove(ClassProperty);
		}
		else
		{
			FloatingPropertiesSettings->PropertyAnchors.FindOrAdd(ClassProperty) = PropertyAnchor;
		}

		FloatingPropertiesSettings->PropertyPositions.Remove(ClassProperty);
	}
	// No parent so use free position
	else
	{
		const FVector2f StackSize = GetStackSize();

		if (FMath::IsNearlyZero(StackSize.X) || FMath::IsNearlyZero(StackSize.Y)
			|| StackSize.X < 0 || StackSize.Y < 0)
		{
			return EFloatingPropertiesUpdateResult::Failed;
		}

		const FVector2f AvailableDragArea = DraggableArea - StackSize;
		const FVector2f AnchorMultiplier = CalculateAnchorMultiplier(PropertyPosition);

		FVector2f Position = {
			(AvailableDragArea.X * AnchorMultiplier.X) + static_cast<float>(PropertyPosition.Offset.X),
			(AvailableDragArea.Y * AnchorMultiplier.Y) + static_cast<float>(PropertyPosition.Offset.Y)
		};

		Position.X = FMath::Clamp(Position.X, 0.f, AvailableDragArea.X);
		Position.Y = FMath::Clamp(Position.Y, 0.f, AvailableDragArea.Y);

		SetCachedPosition(Position);

		if (PropertyPosition.OnDefaultStack())
		{
			FloatingPropertiesSettings->PropertyPositions.Remove(ClassProperty);
		}
		else
		{
			FloatingPropertiesSettings->PropertyPositions.FindOrAdd(ClassProperty) = PropertyPosition;
		}
		
		FloatingPropertiesSettings->PropertyAnchors.Remove(ClassProperty);
	}

	return EFloatingPropertiesUpdateResult::Updated;
}

FVector2f FFloatingPropertiesPropertyNode::CalculateAnchorMultiplier(const FFloatingPropertiesClassPropertyPosition& InPropertyPosition)
{
	FVector2f AnchorMultiplier = FVector2f::ZeroVector;

	switch (InPropertyPosition.HorizontalAnchor.GetValue())
	{
		case EHorizontalAlignment::HAlign_Fill:
		case EHorizontalAlignment::HAlign_Left:
			AnchorMultiplier.X = 0.0f;
			break;

		case EHorizontalAlignment::HAlign_Center:
			AnchorMultiplier.X = 0.5f;
			break;

		case EHorizontalAlignment::HAlign_Right:
			AnchorMultiplier.X = 1.0f;
			break;
	}

	switch (InPropertyPosition.VerticalAnchor.GetValue())
	{
		case EVerticalAlignment::VAlign_Fill:
		case EVerticalAlignment::VAlign_Top:
			AnchorMultiplier.Y = 0.0f;
			break;

		case EVerticalAlignment::VAlign_Center:
			AnchorMultiplier.Y = 0.5f;
			break;

		case EVerticalAlignment::VAlign_Bottom:
			AnchorMultiplier.Y = 1.0f;
			break;
	}

	return AnchorMultiplier;
}
