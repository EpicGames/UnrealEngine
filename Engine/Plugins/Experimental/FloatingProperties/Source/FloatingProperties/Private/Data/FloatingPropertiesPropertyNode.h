// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FloatingPropertiesSettings.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FloatingPropertiesPropertyNodeContainer;
class SFloatingPropertiesPropertyWidget;

enum class EFloatingPropertiesUpdateResult : uint8
{
	Failed = 0,
	AlreadyUpToDate,
	Updated
};

namespace UE::FloatingProperties
{
	EFloatingPropertiesUpdateResult CombineUpdateResult(EFloatingPropertiesUpdateResult InCurrentResult, EFloatingPropertiesUpdateResult InNewResult);
}

class FFloatingPropertiesPropertyNode : public TSharedFromThis<FFloatingPropertiesPropertyNode>
{
public:
	FFloatingPropertiesPropertyNode(FloatingPropertiesPropertyNodeContainer* InLinkedList,
		TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget, int32 InIndex);

	bool operator==(const FFloatingPropertiesPropertyNode& InOther) const
	{
		return MyIndex == InOther.MyIndex;
	}

	bool HasParent() const;

	bool HasParent(TSharedRef<FFloatingPropertiesPropertyNode> InParent) const;

	TSharedPtr<FFloatingPropertiesPropertyNode> GetParent() const;

	bool HasChild() const;

	bool HasChild(TSharedRef<FFloatingPropertiesPropertyNode> InChild) const;

	TSharedPtr<FFloatingPropertiesPropertyNode> GetChild() const;

	bool HasSibling(TSharedRef<FFloatingPropertiesPropertyNode> InSibling) const;

	/* Add a parent relationship. */
	void SetParent(TSharedPtr<FFloatingPropertiesPropertyNode> InParent);

	/** Add child  relationship. */
	void SetChild(TSharedRef<FFloatingPropertiesPropertyNode> InChild);

	/** Insert between this parent and its child. */
	void InsertAfter(TSharedRef<FFloatingPropertiesPropertyNode> InMiddle);

	/** Remove parent and return it. */
	TSharedPtr<FFloatingPropertiesPropertyNode> RemoveParent();

	/** Remove child and return it. */
	TSharedPtr<FFloatingPropertiesPropertyNode> RemoveChild();

	void RemoveFromStack();

	TSharedRef<FFloatingPropertiesPropertyNode> GetStackRootNode() const;

	TSharedRef<FFloatingPropertiesPropertyNode> GetStackLeafMostNode() const;

	const FFloatingPropertiesClassPropertyAnchor& GetPropertyAnchor() const;

	const FFloatingPropertiesClassPropertyPosition& GetPropertyPosition() const;

	TSharedRef<SFloatingPropertiesPropertyWidget> GetPropertyWidget() const;

	/** Returns true if the node was able to update its position or if it already had a valid one. */
	EFloatingPropertiesUpdateResult UpdatePropertyNodePosition();

	/** Returns true if all nodes in the stack were able to update their positions. */
	EFloatingPropertiesUpdateResult UpdateStackPositions(bool bInInvalidate);

	void InvalidateCachedPosition();

	const FVector2f& GetCachedPosition() const;

	void SetPropertyPositionDirect(const FFloatingPropertiesClassPropertyPosition& InPropertyPosition);

	void SetPropertyPosition(const FVector2f& InDraggableArea, const FVector2f& InPosition);

	TArray<TSharedRef<FFloatingPropertiesPropertyNode>> GetNodeStack() const;

	FVector2f GetStackSize() const;

protected:
	static FVector2f CalculateAnchorMultiplier(const FFloatingPropertiesClassPropertyPosition& InPropertyPosition);

	static FFloatingPropertiesClassPropertyPosition CalculatePropertyPosition(const FVector2f& InDraggableArea,
		const FVector2f& InStackSize, const FVector2f& InPosition);

	FloatingPropertiesPropertyNodeContainer* LinkedList;

	int32 MyIndex;

	int32 ParentNodeIndex;

	int32 ChildNodeIndex;

	TSharedRef<SFloatingPropertiesPropertyWidget> PropertyWidget;

	FFloatingPropertiesClassPropertyAnchor PropertyAnchor;

	FFloatingPropertiesClassPropertyPosition PropertyPosition;

	mutable FVector2f CachedPosition;

	mutable bool bValidPosition;

	void SetCachedPosition(const FVector2f& InPosition);
};
