// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/FloatingPropertiesPropertyNodeContainer.h"
#include "Data/FloatingPropertiesPropertyNode.h"
#include "Math/Vector2D.h"
#include "Widgets/SFloatingPropertiesPropertyWidget.h"
#include "Widgets/SFloatingPropertiesViewportWidget.h"

FloatingPropertiesPropertyNodeContainer::FloatingPropertiesPropertyNodeContainer(TSharedRef<SFloatingPropertiesViewportWidget> InViewportWidget)
{
	ContainerWeak = InViewportWidget;
}

FVector2f FloatingPropertiesPropertyNodeContainer::GetDraggableArea() const
{
	if (TSharedPtr<SFloatingPropertiesViewportWidget> ContainerWidget = ContainerWeak.Pin())
	{
		return ContainerWidget->GetDraggableArea();
	}

	return FVector2f::ZeroVector;
}

TSharedPtr<FFloatingPropertiesPropertyNode> FloatingPropertiesPropertyNodeContainer::FindNodeForWidget(
	TSharedRef<SFloatingPropertiesPropertyWidget> InWidget) const
{
	const int32* NodeIndexPtr = WidgetToNode.Find(InWidget);

	if (NodeIndexPtr)
	{
		return PropertyNodes[*NodeIndexPtr];
	}

	return nullptr;
}

TSharedRef<FFloatingPropertiesPropertyNode> FloatingPropertiesPropertyNodeContainer::AddWidget(TSharedRef<SFloatingPropertiesPropertyWidget> InWidget)
{
	const int32 PropertyIndex = PropertyNodes.Num();

	TSharedRef<FFloatingPropertiesPropertyNode> Node = MakeShared<FFloatingPropertiesPropertyNode>(this, InWidget, PropertyIndex);

	PropertyNodes.Add(Node);
	WidgetToNode.Add(InWidget, PropertyIndex);

	return Node;
}

void FloatingPropertiesPropertyNodeContainer::Reserve(int32 InAmount)
{
	WidgetToNode.Reserve(InAmount);
	PropertyNodes.Reserve(InAmount);
}

void FloatingPropertiesPropertyNodeContainer::Empty()
{
	WidgetToNode.Empty();
	PropertyNodes.Empty();
}

void FloatingPropertiesPropertyNodeContainer::InvalidateAllPositions() const
{
	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : PropertyNodes)
	{
		Node->InvalidateCachedPosition();
	}
}

EFloatingPropertiesUpdateResult FloatingPropertiesPropertyNodeContainer::EnsureCachedPositions() const
{
	using namespace UE::FloatingProperties;

	EFloatingPropertiesUpdateResult Result = EFloatingPropertiesUpdateResult::AlreadyUpToDate;

	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : PropertyNodes)
	{
		Result = CombineUpdateResult(Result, Node->UpdatePropertyNodePosition());
	}

	return Result;
}
