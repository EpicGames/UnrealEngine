// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FloatingPropertiesSettings.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"

class FFloatingPropertiesPropertyNode;
class SFloatingPropertiesPropertyWidget;
class SFloatingPropertiesViewportWidget;
enum class EFloatingPropertiesUpdateResult : uint8;

class FloatingPropertiesPropertyNodeContainer
{
public:
	FloatingPropertiesPropertyNodeContainer(TSharedRef<SFloatingPropertiesViewportWidget> InViewportWidget);

	TConstArrayView<TSharedRef<FFloatingPropertiesPropertyNode>> GetNodes() const { return PropertyNodes; }

	void Reserve(int32 InAmount);

	TSharedRef<FFloatingPropertiesPropertyNode> AddWidget(TSharedRef<SFloatingPropertiesPropertyWidget> InWidget);

	TSharedPtr<FFloatingPropertiesPropertyNode> FindNodeForWidget(TSharedRef<SFloatingPropertiesPropertyWidget> InWidget) const;

	void Empty();

	void InvalidateAllPositions() const;

	EFloatingPropertiesUpdateResult EnsureCachedPositions() const;

	FVector2f GetDraggableArea() const;

protected:
	TWeakPtr<SFloatingPropertiesViewportWidget> ContainerWeak;

	TArray<TSharedRef<FFloatingPropertiesPropertyNode>> PropertyNodes;

	TMap<TWeakPtr<SFloatingPropertiesPropertyWidget>, int32> WidgetToNode;
};