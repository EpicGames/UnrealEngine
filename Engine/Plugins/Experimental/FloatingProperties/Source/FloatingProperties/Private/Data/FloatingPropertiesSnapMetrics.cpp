// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/FloatingPropertiesSnapMetrics.h"
#include "Data/FloatingPropertiesPropertyNode.h"
#include "Widgets/SFloatingPropertiesPropertyWidget.h"

FFloatingPropertiesSnapMetrics FFloatingPropertiesSnapMetrics::Make(TSharedRef<FFloatingPropertiesPropertyNode> InNode,
	const FVector2f& InSnapPosition, EFloatingPropertiesSnapType InAttachType)
{
	FFloatingPropertiesSnapMetrics Metrics = {
		InNode,
		GetSnapPosition(InNode, InAttachType),
		0.f,
		InAttachType
	};

	Metrics.DistanceSq = (Metrics.Position - InSnapPosition).SizeSquared();

	return Metrics;
}

FVector2f FFloatingPropertiesSnapMetrics::GetSnapPosition(TSharedRef<FFloatingPropertiesPropertyNode> InNode, EFloatingPropertiesSnapType InAttachType)
{
	FVector2f Position = InNode->GetPropertyWidget()->GetTickSpaceGeometry().GetAbsolutePosition();

	if (InAttachType == EFloatingPropertiesSnapType::AttachAsParent)
	{
		Position.Y += InNode->GetPropertyWidget()->GetAbsolutePropertySize().Y;
	}

	return Position;
}

void FFloatingPropertiesSnapMetrics::SnapToDraggableArea(const FVector2f& InDraggableArea, FVector2f& InOutPosition)
{
	if (InOutPosition.X <= FFloatingPropertiesSnapMetrics::SnapDistance)
	{
		InOutPosition.X = 0.f;
	}
	else if (InOutPosition.X >= (InDraggableArea.X - FFloatingPropertiesSnapMetrics::SnapDistance))
	{
		InOutPosition.X = InDraggableArea.X;
	}

	if (InOutPosition.Y <= FFloatingPropertiesSnapMetrics::SnapDistance)
	{
		InOutPosition.Y = 0.f;
	}
	else if (InOutPosition.Y >= (InDraggableArea.Y - FFloatingPropertiesSnapMetrics::SnapDistance))
	{
		InOutPosition.Y = InDraggableArea.Y;
	}
}
