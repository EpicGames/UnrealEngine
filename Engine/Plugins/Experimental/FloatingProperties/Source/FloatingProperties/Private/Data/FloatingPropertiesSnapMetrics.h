// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FFloatingPropertiesPropertyNode;

enum class EFloatingPropertiesSnapType
{
	AttachAsParent,
	AttachAsChild
};

struct FFloatingPropertiesSnapMetrics
{
	static constexpr int32 TopLeft = 0;
	static constexpr int32 BottomLeft = 1;
	static constexpr float SnapDistance = 10.f;
	static constexpr float SnapDistanceSq = SnapDistance * SnapDistance;
	static constexpr float SnapBreakDistanceSq = SnapDistanceSq * 2.f;

	static FFloatingPropertiesSnapMetrics Make(TSharedRef<FFloatingPropertiesPropertyNode> InNode,
		const FVector2f& InSnapPosition, EFloatingPropertiesSnapType InAttachType);

	static FVector2f GetSnapPosition(TSharedRef<FFloatingPropertiesPropertyNode> InNode, EFloatingPropertiesSnapType InAttachType);

	static void SnapToDraggableArea(const FVector2f& InDraggableArea, FVector2f& InOutPosition);

	TSharedRef<FFloatingPropertiesPropertyNode> Node;
	FVector2f Position;
	float DistanceSq;
	EFloatingPropertiesSnapType AttachType;
};
