// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/FloatingPropertiesWidgetController.h"
#include "Algo/Reverse.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Data/IFloatingPropertiesWidgetContainer.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "FloatingPropertiesModule.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Selection.h"
#include "UObject/Class.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "Widgets/SFloatingPropertiesViewportWidget.h"

namespace UE::FloatingFavorite::Private
{
	const TSet<FFieldClass*> AllowedClasses = {
		FBoolProperty::StaticClass(),
		FEnumProperty::StaticClass(),
		FNameProperty::StaticClass(),
		FNumericProperty::StaticClass(),
		FObjectPropertyBase::StaticClass(),
		FStrProperty::StaticClass(),
		FTextProperty::StaticClass(),
		FStructProperty::StaticClass()
	};

	bool IsValidPropertyType(const FProperty& InProperty)
	{
		FFieldClass* PropertyClass = InProperty.GetClass();

		while (true)
		{
			if (AllowedClasses.Contains(PropertyClass))
			{
				return true;
			}

			PropertyClass = PropertyClass->GetSuperClass();

			if (!PropertyClass || PropertyClass == FProperty::StaticClass())
			{
				return false;
			}
		}
	}

	bool IsFullPropertyStruct(const FStructProperty& InProperty)
	{
		const FFloatingPropertiesModule::FCreateStructPropertyValueWidgetDelegate* Delegate
			= FFloatingPropertiesModule::Get().GetStructPropertyValueWidgetDelegate(InProperty.Struct);

		return (Delegate && Delegate->IsBound());
	}
}

FFloatingPropertiesWidgetController::FFloatingPropertiesWidgetController(TSharedRef<IFloatingPropertiesDataProvider> InDataProvider)
	: DataProvider(InDataProvider)
{
}

FFloatingPropertiesWidgetController::~FFloatingPropertiesWidgetController()
{
	UnregisterSelectionChanged();
	RemoveWidgets();
}

void FFloatingPropertiesWidgetController::Init()
{
	RegisterSelectionChanged();
	OnViewportChange();
}

void FFloatingPropertiesWidgetController::OnViewportChange()
{
	const TArray<TSharedRef<IFloatingPropertiesWidgetContainer>>& Containers = DataProvider->GetWidgetContainers();

	bool bCreatedWidget = false;

	for (const TSharedRef<IFloatingPropertiesWidgetContainer>& Container : Containers)
	{
		if (Container->GetWidget().IsValid())
		{
			continue;
		}

		Container->SetWidget(CreateWidget(FIsVisibleDelegate::CreateSPLambda(this,
			[this, Container]()
			{
				return DataProvider->IsWidgetVisibleInContainer(Container);
			})));

		bCreatedWidget = true;
	}

	if (bCreatedWidget)
	{
		SelectedActorWeak.Reset();
		SelectedComponentWeak.Reset();
		OnSelectionChange(nullptr);
	}
}

void FFloatingPropertiesWidgetController::RegisterSelectionChanged()
{
	if (USelection* ActorSelection = DataProvider->GetActorSelection())
	{
		if (UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet())
		{
			SelectionSet->OnChanged().AddSP(this, &FFloatingPropertiesWidgetController::OnSelectionChange);
		}
	}

	if (USelection* ComponentSelection = DataProvider->GetComponentSelection())
	{
		if (UTypedElementSelectionSet* SelectionSet = ComponentSelection->GetElementSelectionSet())
		{
			SelectionSet->OnChanged().AddSP(this, &FFloatingPropertiesWidgetController::OnSelectionChange);
		}
	}
}

void FFloatingPropertiesWidgetController::UnregisterSelectionChanged()
{
	if (DataProvider.IsValid())
	{
		if (USelection* ActorSelection = DataProvider->GetActorSelection())
		{
			if (UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet())
			{
				SelectionSet->OnChanged().RemoveAll(this);
			}
		}

		if (USelection* ComponentSelection = DataProvider->GetComponentSelection())
		{
			if (UTypedElementSelectionSet* SelectionSet = ComponentSelection->GetElementSelectionSet())
			{
				SelectionSet->OnChanged().RemoveAll(this);
			}
		}
	}
}

void FFloatingPropertiesWidgetController::RemoveWidgets()
{
	for (const TSharedRef<IFloatingPropertiesWidgetContainer>& Container : DataProvider->GetWidgetContainers())
	{
		Container->RemoveWidget();
	}
}

void FFloatingPropertiesWidgetController::OnSelectionChange(const UTypedElementSelectionSet* InSelectionSet)
{
	OnSelectionChange();
}

void FFloatingPropertiesWidgetController::OnSelectionChange()
{
	OnSelectionChange(
		DataProvider->GetWorld(),
		DataProvider->GetActorSelection(),
		DataProvider->GetComponentSelection()
	);
}

void FFloatingPropertiesWidgetController::OnSelectionChange(UWorld* InWorld, USelection* InActorSelection, 
	USelection* InComponentSelection)
{
	if (!InWorld || !InActorSelection || !InComponentSelection)
	{
		return;
	}

	AActor* SelectedActor = nullptr;
	UActorComponent* SelectedComponent = nullptr;

	TArray<UActorComponent*> SelectedComponents;
	InComponentSelection->GetSelectedObjects(SelectedComponents);

	for (UActorComponent* Component : SelectedComponents)
	{
		if (IsValid(Component) && Component->GetWorld() == InWorld)
		{
			if (AActor* Owner = Component->GetOwner())
			{
				if (Owner->GetRootComponent() != Component)
				{
					SelectedComponent = Component;
					break;
				}
			}
		}
	}

	if (SelectedComponent == nullptr)
	{
		TArray<AActor*> SelectedActors;
		InActorSelection->GetSelectedObjects(SelectedActors);

		for (AActor* Actor : SelectedActors)
		{
			if (IsValid(Actor) && Actor->GetWorld() == InWorld)
			{
				SelectedActor = Actor;
				break;
			}
		}
	}

	if (SelectedComponentWeak.Get() == SelectedComponent && SelectedActorWeak.Get() == SelectedActor)
	{
 		return;
	}

	SelectedActorWeak = SelectedActor;
	ActorProperties.Properties.Empty();
	ActorProperties.Generator.Reset();
	UpdatePropertiesForObject(SelectedActor, ActorProperties);

	SelectedComponentWeak = SelectedComponent;
	ComponentProperties.Properties.Empty();
	ComponentProperties.Generator.Reset();
	UpdatePropertiesForObject(SelectedComponent, ComponentProperties);

	RebuildWidgets(ActorProperties, ComponentProperties);
}

void FFloatingPropertiesWidgetController::UpdatePropertiesForObject(UObject* InObject, FFloatingPropertiesPropertyList& OutPropertyList)
{
	OutPropertyList.Generator.Reset();
	OutPropertyList.Properties.Empty();

	if (!InObject)
	{
		return;
	}

	const bool bObjectIsActor = InObject->IsA<AActor>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FPropertyRowGeneratorArgs Args;
	Args.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	Args.bAllowEditingClassDefaultObjects = false;
	Args.bAllowMultipleTopLevelObjects = false;
	Args.bShouldShowHiddenProperties = false;

	TSharedRef<IPropertyRowGenerator> PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->SetObjects({InObject});

	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator->GetRootTreeNodes();
	TArray<TSharedRef<IDetailTreeNode>> Stack(RootNodes);

	Algo::Reverse(Stack);

	Stack.Reserve(100); // There's going to be a lot.

	while (!Stack.IsEmpty())
	{
		TSharedRef<IDetailTreeNode> Node = Stack.Pop(EAllowShrinking::No);

		if (Node->GetNodeType() == EDetailNodeType::Item)
		{
			if (TSharedPtr<IPropertyHandle> PropertyHandle = Node->CreatePropertyHandle())
			{
				FProperty* Property = PropertyHandle->GetProperty();

				if (!Property)
				{
					continue;
				}

				using namespace UE::FloatingFavorite::Private;

				if (!IsValidPropertyType(*Property))
				{
					continue;
				}

				if (PropertyHandle->IsFavorite())
				{
					OutPropertyList.Properties.Add({PropertyHandle, Property, Node});
				}
			}
		}

		TArray<TSharedRef<IDetailTreeNode>> Children;
		Node->GetChildren(Children, true);

		Algo::Reverse(Children);

		Stack.Append(Children);
	}

	if (!OutPropertyList.Properties.IsEmpty())
	{
		OutPropertyList.Generator = PropertyRowGenerator;
	}
}

TSharedRef<SFloatingPropertiesViewportWidget> FFloatingPropertiesWidgetController::CreateWidget(FIsVisibleDelegate InVisibilityCallback)
{
	return SNew(SFloatingPropertiesViewportWidget)
		.IsVisible(InVisibilityCallback);
}

void FFloatingPropertiesWidgetController::RebuildWidgets(const FFloatingPropertiesPropertyList& InActorProperties,
	const FFloatingPropertiesPropertyList& InComponentProperties)
{
	for (const TSharedRef<IFloatingPropertiesWidgetContainer>& Container : DataProvider->GetWidgetContainers())
	{
		if (TSharedPtr<SFloatingPropertiesViewportWidget> Widget = Container->GetWidget())
		{
			Widget->BuildWidget(
				SelectedActorWeak.Get(),
				InActorProperties,
				SelectedComponentWeak.Get(),
				InComponentProperties
			);
		}
	}
}
