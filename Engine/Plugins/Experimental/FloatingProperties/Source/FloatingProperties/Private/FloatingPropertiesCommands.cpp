// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatingPropertiesCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FloatingPropertiesCommands"

FFloatingPropertiesCommands::FFloatingPropertiesCommands()
	: TCommands<FFloatingPropertiesCommands>(TEXT("FloatingProperties")
		, LOCTEXT("FloatingProperties", "FloatingProperties")
		, NAME_None
		, FAppStyle::GetAppStyleSetName()
	)
{
}

void FFloatingPropertiesCommands::RegisterCommands()
{
	UI_COMMAND(ToggleEnabled
		, "Floating Properties"
		, "Toggle floating properties on and off."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());	
}

#undef LOCTEXT_NAMESPACE
