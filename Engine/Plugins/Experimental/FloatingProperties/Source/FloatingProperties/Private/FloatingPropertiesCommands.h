// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FFloatingPropertiesCommands : public TCommands<FFloatingPropertiesCommands>
{
public:
	FFloatingPropertiesCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> ToggleEnabled;
};
