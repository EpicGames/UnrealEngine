// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatingPropertiesModule.h"
#include "FloatingPropertiesCommands.h"
#include "FloatingPropertiesSettings.h"
#include "LevelEditor.h"
#include "LevelEditor/FloatingPropertiesLevelEditorWidgetController.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Widgets/CustomPropertyWidgets/SFloatingPropertiesColorPropertyEditor.h"
#include "Widgets/CustomPropertyWidgets/SFloatingPropertiesLinearColorPropertyEditor.h"

DEFINE_LOG_CATEGORY(LogFloatingProperties);

FFloatingPropertiesModule& FFloatingPropertiesModule::Get()
{
	return FModuleManager::GetModuleChecked<FFloatingPropertiesModule>("FloatingProperties");
}

void FFloatingPropertiesModule::StartupModule()
{
	FFloatingPropertiesCommands::Register();

	UFloatingPropertiesSettings::OnChange.AddRaw(this, &FFloatingPropertiesModule::OnSettingsChanged);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnLevelEditorCreated().AddRaw(this, &FFloatingPropertiesModule::OnLevelEditorCreated);

	AddDefaultStructPropertyValueWidgetDelegates();

	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FFloatingPropertiesModule::OnEnginePreExit);
}

void FFloatingPropertiesModule::ShutdownModule()
{
	FFloatingPropertiesCommands::Unregister();

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnLevelEditorCreated().RemoveAll(this);
	}

	DestroyLevelEditorWidgetController();

	UFloatingPropertiesSettings::OnChange.RemoveAll(this);

	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

void FFloatingPropertiesModule::RegiserStructPropertyValueWidgetDelegate(UScriptStruct* InStruct,
	FCreateStructPropertyValueWidgetDelegate InDelegate)
{
	if (!InStruct)
	{
		return;
	}

	if (!InDelegate.IsBound())
	{
		return;
	}

	const FName StructName = InStruct->GetFName();

	PropertyValueWidgetDelegates.Add(StructName, InDelegate);
}

const FFloatingPropertiesModule::FCreateStructPropertyValueWidgetDelegate* FFloatingPropertiesModule::GetStructPropertyValueWidgetDelegate(
	UScriptStruct* InStruct) const
{
	if (!InStruct)
	{
		return nullptr;
	}

	const FName StructName = InStruct->GetFName();

	return PropertyValueWidgetDelegates.Find(StructName);
}

void FFloatingPropertiesModule::UnregiserStructPropertyValueWidgetDelegate(UScriptStruct* InStruct)
{
	if (!InStruct)
	{
		return;
	}

	const FName StructName = InStruct->GetFName();

	PropertyValueWidgetDelegates.Remove(StructName);
}

void FFloatingPropertiesModule::OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
{
	CreateLevelEditorWidgetController(InLevelEditor);
}

void FFloatingPropertiesModule::CreateLevelEditorWidgetController(TSharedPtr<ILevelEditor> InLevelEditor)
{
	if (LevelEditorWidgetController.IsValid())
	{
		return;
	}

	if (!InLevelEditor.IsValid())
	{
		return;
	}

	FFloatingPropertiesLevelEditorWidgetController::StaticInit();

	if (const UFloatingPropertiesSettings* FloatingPropertiesSettings = GetDefault<UFloatingPropertiesSettings>())
	{
		if (FloatingPropertiesSettings->bEnabled)
		{
			LevelEditorWidgetController = MakeShared<FFloatingPropertiesLevelEditorWidgetController>(InLevelEditor.ToSharedRef());
			LevelEditorWidgetController->Init();
		}
	}	
}

void FFloatingPropertiesModule::DestroyLevelEditorWidgetController()
{
	LevelEditorWidgetController.Reset();
}

void FFloatingPropertiesModule::OnSettingsChanged(const UFloatingPropertiesSettings* InSettings, FName InSetting)
{
	static const FName EnabledName = GET_MEMBER_NAME_CHECKED(UFloatingPropertiesSettings, bEnabled);

	if (InSetting == EnabledName)
	{
		if (InSettings->bEnabled)
		{
			if (!LevelEditorWidgetController.IsValid())
			{
				FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
				TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();

				if (LevelEditor.IsValid())
				{
					CreateLevelEditorWidgetController(LevelEditor.ToSharedRef());
				}
			}
		}
		else
		{
			if (LevelEditorWidgetController.IsValid())
			{
				DestroyLevelEditorWidgetController();
			}
		}
	}
}

void FFloatingPropertiesModule::AddDefaultStructPropertyValueWidgetDelegates()
{
	RegiserStructPropertyValueWidgetDelegate(
		TBaseStructure<FColor>::Get(),
		FCreateStructPropertyValueWidgetDelegate::CreateStatic(&SFloatingPropertiesColorPropertyEditor::CreateWidget)
	);

	RegiserStructPropertyValueWidgetDelegate(
		TBaseStructure<FLinearColor>::Get(),
		FCreateStructPropertyValueWidgetDelegate::CreateStatic(&SFloatingPropertiesLinearColorPropertyEditor::CreateWidget)
	);
}

void FFloatingPropertiesModule::OnEnginePreExit()
{
	PropertyValueWidgetDelegates.Empty();
	LevelEditorWidgetController.Reset();
}

IMPLEMENT_MODULE(FFloatingPropertiesModule, FloatingProperties)
