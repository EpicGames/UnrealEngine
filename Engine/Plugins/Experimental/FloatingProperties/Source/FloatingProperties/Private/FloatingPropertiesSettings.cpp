// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatingPropertiesSettings.h"
#include "Containers/UnrealString.h"

UFloatingPropertiesSettings::FOnFloatingPropertiesSettingsChanged UFloatingPropertiesSettings::OnChange;

const FFloatingPropertiesClassPropertyPosition FFloatingPropertiesClassPropertyPosition::DefaultStackPosition = {
		EHorizontalAlignment::HAlign_Left,
		EVerticalAlignment::VAlign_Bottom,
		FIntPoint::ZeroValue
};

bool FFloatingPropertiesClassPropertyPosition::OnDefaultStack() const
{
	return HorizontalAnchor == DefaultStackPosition.HorizontalAnchor
		&& VerticalAnchor == DefaultStackPosition.VerticalAnchor
		&& Offset == DefaultStackPosition.Offset;
}

FString UFloatingPropertiesSettings::FindUniqueName(const FFloatingPropertiesClassProperties& InPropertyList, const FString& InDefault)
{
	if (!InPropertyList.Properties.Contains(InDefault))
	{
		return InDefault;
	}

	for (int32 Index = 1; true; ++Index)
	{
		FString UniqueTest = InDefault + "_" + FString::FromInt(Index);

		if (!InPropertyList.Properties.Contains(UniqueTest))
		{
			return UniqueTest;
		}
	}
}

UFloatingPropertiesSettings::UFloatingPropertiesSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("Floating Properties");

	bEnabled = true;
}

void UFloatingPropertiesSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName != NAME_None)
	{
		OnChange.Broadcast(this, MemberName);
	}
}
