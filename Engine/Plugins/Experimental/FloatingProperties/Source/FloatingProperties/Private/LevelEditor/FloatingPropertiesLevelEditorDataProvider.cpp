// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatingPropertiesLevelEditorDataProvider.h"
#include "LevelEditor.h"
#include "LevelEditor/FloatingPropertiesLevelEditorWidgetContainer.h"
#include "Selection.h"
#include "SLevelViewport.h"

FFloatingPropertiesLevelEditorDataProvider::FFloatingPropertiesLevelEditorDataProvider(TSharedRef<ILevelEditor> InLevelEditor)
	: FFloatingPropertiesToolkitHostDataProvider(InLevelEditor)
{
}

TArray<TSharedRef<IFloatingPropertiesWidgetContainer>> FFloatingPropertiesLevelEditorDataProvider::GetWidgetContainers()
{
	ClearExpiredViewports();

	TSharedPtr<ILevelEditor> LevelEditor = StaticCastSharedPtr<ILevelEditor>(ToolkitHostWeak.Pin());

	if (!LevelEditor.IsValid())
	{
		return {};
	}

	TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();

	TArray<TSharedRef<IFloatingPropertiesWidgetContainer>> Containers;
	Containers.Reserve(Viewports.Num());

	for (const TSharedPtr<SLevelViewport>& LevelViewport : Viewports)
	{
		if (LevelViewport.IsValid())
		{
			if (const TSharedRef<FFloatingPropertiesLevelEditorWidgetContainer>* Container = CachedContainers.Find(LevelViewport))
			{
				Containers.Add(*Container);
			}
			else
			{
				TSharedRef<FFloatingPropertiesLevelEditorWidgetContainer> NewContainer = MakeShared<FFloatingPropertiesLevelEditorWidgetContainer>(LevelViewport.ToSharedRef());
				CachedContainers.Add(LevelViewport, NewContainer);
				Containers.Add(NewContainer);
			}
		}
	}

	return Containers;
}

bool FFloatingPropertiesLevelEditorDataProvider::IsWidgetVisibleInContainer(TSharedRef<IFloatingPropertiesWidgetContainer> InContainer) const
{
	TSharedPtr<ILevelEditor> LevelEditor = StaticCastSharedPtr<ILevelEditor>(ToolkitHostWeak.Pin());

	if (!LevelEditor.IsValid())
	{
		return false;
	}

	TSharedRef<FFloatingPropertiesLevelEditorWidgetContainer> LevelEditorWidgetContainer = StaticCastSharedRef<FFloatingPropertiesLevelEditorWidgetContainer>(InContainer);

	return LevelEditor->GetActiveViewportInterface() == LevelEditorWidgetContainer->GetLevelViewport();
}

void FFloatingPropertiesLevelEditorDataProvider::ClearExpiredViewports()
{
	for (auto Iter = CachedContainers.CreateIterator(); Iter; ++Iter)
	{
		const TWeakPtr<SLevelViewport>& Viewport = Iter.Key();

		if (!Viewport.IsValid())
		{
			Iter.RemoveCurrent();
		}
	}
}
