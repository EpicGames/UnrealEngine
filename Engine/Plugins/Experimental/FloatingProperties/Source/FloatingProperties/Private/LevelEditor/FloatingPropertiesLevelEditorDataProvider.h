// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolkitHost/FloatingPropertiesToolkitHostDataProvider.h"
#include "Containers/Map.h"
#include "Data/IFloatingPropertiesWidgetContainer.h"
#include "Templates/SharedPointer.h"

class FFloatingPropertiesLevelEditorWidgetContainer;
class ILevelEditor;
class SLevelViewport;

class FFloatingPropertiesLevelEditorDataProvider : public FFloatingPropertiesToolkitHostDataProvider
{
public:
	FFloatingPropertiesLevelEditorDataProvider(TSharedRef<ILevelEditor> InLevelEditor);
	virtual ~FFloatingPropertiesLevelEditorDataProvider() = default;

	//~ Begin IFloatingPropertiesDataProvider
	virtual TArray<TSharedRef<IFloatingPropertiesWidgetContainer>> GetWidgetContainers() override;
	virtual bool IsWidgetVisibleInContainer(TSharedRef<IFloatingPropertiesWidgetContainer> InContainer) const override;
	//~ End IFloatingPropertiesDataProvider

protected:
	TMap<TWeakPtr<SLevelViewport>, TSharedRef<FFloatingPropertiesLevelEditorWidgetContainer>> CachedContainers;

	void ClearExpiredViewports();
};
