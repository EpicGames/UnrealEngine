// Copyright Epic Games, Inc. All Rights Reserved.

#include "FloatingPropertiesLevelEditorWidgetContainer.h"
#include "SLevelViewport.h"
#include "Widgets/SFloatingPropertiesViewportWidget.h"

FFloatingPropertiesLevelEditorWidgetContainer::FFloatingPropertiesLevelEditorWidgetContainer(TSharedRef<SLevelViewport> InLevelViewport)
	: LevelViewportWeak(InLevelViewport)
{
}

FFloatingPropertiesLevelEditorWidgetContainer::~FFloatingPropertiesLevelEditorWidgetContainer()
{
	RemoveWidget();
}

bool FFloatingPropertiesLevelEditorWidgetContainer::SetWidget(TSharedRef<SFloatingPropertiesViewportWidget> InWidget)
{
	if (TSharedPtr<SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
	{
		Widget = InWidget;
		LevelViewport->AddOverlayWidget(InWidget->AsWidget());
		return true;
	}

	return false;
}

TSharedPtr<SFloatingPropertiesViewportWidget> FFloatingPropertiesLevelEditorWidgetContainer::GetWidget() const
{
	return Widget;
}

bool FFloatingPropertiesLevelEditorWidgetContainer::RemoveWidget()
{
	if (TSharedPtr<SLevelViewport> LevelViewport = LevelViewportWeak.Pin())
	{
		if (Widget.IsValid())
		{
			LevelViewport->RemoveOverlayWidget(Widget->AsWidget());
			Widget.Reset();
			return true;
		}
	}

	return false;
}

bool FFloatingPropertiesLevelEditorWidgetContainer::IsValid() const
{
	return LevelViewportWeak.IsValid();
}

TSharedPtr<SLevelViewport> FFloatingPropertiesLevelEditorWidgetContainer::GetLevelViewport() const
{
	return LevelViewportWeak.Pin();
}
