// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/IFloatingPropertiesWidgetContainer.h"
#include "Templates/SharedPointer.h"

class SLevelViewport;

class FFloatingPropertiesLevelEditorWidgetContainer : public IFloatingPropertiesWidgetContainer
{
public:
	FFloatingPropertiesLevelEditorWidgetContainer(TSharedRef<SLevelViewport> InLevelViewport);
	virtual ~FFloatingPropertiesLevelEditorWidgetContainer();

	//~ Begin IFloatingPropertiesWidgetContainer
	virtual TSharedPtr<SFloatingPropertiesViewportWidget> GetWidget() const override;
	virtual bool SetWidget(TSharedRef<SFloatingPropertiesViewportWidget> InWidget) override;
	virtual bool RemoveWidget() override;
	virtual bool IsValid() const override;
	//~ End IFloatingPropertiesWidgetContainer

	TSharedPtr<SLevelViewport> GetLevelViewport() const;

protected:
	TWeakPtr<SLevelViewport> LevelViewportWeak;
	TSharedPtr<SFloatingPropertiesViewportWidget> Widget;
};
