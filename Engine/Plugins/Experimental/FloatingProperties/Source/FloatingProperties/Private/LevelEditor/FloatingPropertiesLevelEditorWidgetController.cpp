// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditor/FloatingPropertiesLevelEditorWidgetController.h"
#include "Editor.h"
#include "FloatingPropertiesCommands.h"
#include "FloatingPropertiesSettings.h"
#include "LevelEditor.h"
#include "Editor/EditorEngine.h"
#include "LevelEditor/FloatingPropertiesLevelEditorDataProvider.h"
#include "Selection.h"
#include "ToolMenus.h"

void FFloatingPropertiesLevelEditorWidgetController::StaticInit()
{
	RegisterLevelViewportMenuExtensions();
	RegisterLevelEditorCommands();
}

FFloatingPropertiesLevelEditorWidgetController::FFloatingPropertiesLevelEditorWidgetController(TSharedRef<ILevelEditor> InLevelEditor)
	: FFloatingPropertiesWidgetController(MakeShared<FFloatingPropertiesLevelEditorDataProvider>(InLevelEditor))
{
}

FFloatingPropertiesLevelEditorWidgetController::~FFloatingPropertiesLevelEditorWidgetController()
{
	UnregisterLevelViewportClientsChanged();

	if (NextTickTimer.IsValid())
	{
		FTSTicker::RemoveTicker(NextTickTimer);
		NextTickTimer.Reset();
	}	
}

void FFloatingPropertiesLevelEditorWidgetController::Init()
{
	FFloatingPropertiesWidgetController::Init();

	RegisterLevelViewportClientsChanged();
	OnLevelViewportClientsChanged();
}

void FFloatingPropertiesLevelEditorWidgetController::RegisterLevelViewportClientsChanged()
{
	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().AddRaw(this, &FFloatingPropertiesLevelEditorWidgetController::OnLevelViewportClientsChanged);
	}
}

void FFloatingPropertiesLevelEditorWidgetController::UnregisterLevelViewportClientsChanged()
{
	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
}

void FFloatingPropertiesLevelEditorWidgetController::OnLevelViewportClientsChanged()
{
	if (NextTickTimer.IsValid())
	{
		return;
	}

	NextTickTimer = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this,
		[this](float)
		{
			OnViewportChange();
			NextTickTimer.Reset();

			// Do not run again
			return false;
		}),
		/* InDelay - Next frame */ 0.f
	);
}

void FFloatingPropertiesLevelEditorWidgetController::RegisterLevelViewportMenuExtensions()
{
	static const FName OptionsMenuName("LevelEditor.LevelViewportToolBar.Options");
	static const FName OptionsMenuSection("LevelViewportViewportOptions2");

	UToolMenus* const ToolMenus = UToolMenus::Get();

	if (!ToolMenus)
	{
		return;
	}

	UToolMenu* const OptionsMenu = ToolMenus->ExtendMenu(OptionsMenuName);

	if (!OptionsMenu)
	{
		return;
	}

	FToolMenuSection* Section = OptionsMenu->FindSection(OptionsMenuSection);

	if (!Section)
	{
		return;
	}

	const FFloatingPropertiesCommands& Commands = FFloatingPropertiesCommands::Get();

	Section->AddEntry(FToolMenuEntry::InitMenuEntry(
		Commands.ToggleEnabled,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Server")
	));
}

void FFloatingPropertiesLevelEditorWidgetController::RegisterLevelEditorCommands()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	const FFloatingPropertiesCommands& Commands = FFloatingPropertiesCommands::Get();
	
	LevelEditorModule.GetGlobalLevelEditorActions()->MapAction(
		Commands.ToggleEnabled,
		FExecuteAction::CreateLambda(
			[]()
			{
				if (UFloatingPropertiesSettings* Settings = GetMutableDefault<UFloatingPropertiesSettings>())
				{
					Settings->bEnabled = !Settings->bEnabled;
					Settings->SaveConfig();
					
					FPropertyChangedEvent ChangeEvent(
						UFloatingPropertiesSettings::StaticClass()->FindPropertyByName(
							GET_MEMBER_NAME_CHECKED(UFloatingPropertiesSettings, bEnabled)
						),
						EPropertyChangeType::ValueSet
					);
						
					Settings->PostEditChangeProperty(ChangeEvent);
				}
			}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda(
			[]()
			{
				if (const UFloatingPropertiesSettings* Settings = GetDefault<UFloatingPropertiesSettings>())
				{
					return Settings->bEnabled;
				}

				return false;
			})
	);
}
