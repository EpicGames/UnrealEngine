// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/FloatingPropertiesWidgetController.h"
#include "Containers/Ticker.h"

class ILevelEditor;

class FFloatingPropertiesLevelEditorWidgetController : public FFloatingPropertiesWidgetController
{
public:
	static void StaticInit();

	FFloatingPropertiesLevelEditorWidgetController(TSharedRef<ILevelEditor> InLevelEditor);
	virtual ~FFloatingPropertiesLevelEditorWidgetController();

	//~ Begin FFloatingPropertiesWidgetController
	virtual void Init() override;
	//~ End FFloatingPropertiesWidgetController

protected:
	static void RegisterLevelViewportMenuExtensions();

	static void RegisterLevelEditorCommands();

	FTSTicker::FDelegateHandle NextTickTimer;

	void RegisterLevelViewportClientsChanged();

	void UnregisterLevelViewportClientsChanged();

	void OnLevelViewportClientsChanged();
};
