// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolkitHost/FloatingPropertiesToolkitHostDataProvider.h"
#include "EditorModeManager.h"
#include "Toolkits/IToolkitHost.h"

FFloatingPropertiesToolkitHostDataProvider::FFloatingPropertiesToolkitHostDataProvider(TSharedRef<IToolkitHost> InToolkitHost)
{
	ToolkitHostWeak = InToolkitHost;
}

USelection* FFloatingPropertiesToolkitHostDataProvider::GetActorSelection() const
{
	if (TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin())
	{
		return ToolkitHost->GetEditorModeManager().GetSelectedActors();
	}

	return nullptr;
}

USelection* FFloatingPropertiesToolkitHostDataProvider::GetComponentSelection() const
{
	if (TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin())
	{
		return ToolkitHost->GetEditorModeManager().GetSelectedComponents();
	}

	return nullptr;
}

UWorld* FFloatingPropertiesToolkitHostDataProvider::GetWorld() const
{
	if (TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin())
	{
		return ToolkitHost->GetWorld();
	}

	return nullptr;
}
