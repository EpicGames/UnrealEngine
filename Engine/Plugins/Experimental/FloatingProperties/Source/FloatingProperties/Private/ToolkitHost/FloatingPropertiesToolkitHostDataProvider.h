// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/IFloatingPropertiesDataProvider.h"
#include "Templates/SharedPointer.h"

class IToolkitHost;

class FFloatingPropertiesToolkitHostDataProvider : public IFloatingPropertiesDataProvider
{
public:
	FFloatingPropertiesToolkitHostDataProvider(TSharedRef<IToolkitHost> InToolkitHost);
	virtual ~FFloatingPropertiesToolkitHostDataProvider() = default;

	//~ Begin IFloatingPropertiesDataProvider
	virtual USelection* GetActorSelection() const override;
	virtual USelection* GetComponentSelection() const override;
	virtual UWorld* GetWorld() const override;
	//~ End IFloatingPropertiesDataProvider

protected:
	TWeakPtr<IToolkitHost> ToolkitHostWeak;
};
