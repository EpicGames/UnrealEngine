// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/CustomPropertyWidgets/SFloatingPropertiesColorPropertyEditor.h"
#include "PropertyHandle.h"

void SFloatingPropertiesColorPropertyEditor::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

TSharedPtr<SWidget> SFloatingPropertiesColorPropertyEditor::CreateWidget(TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle->GetProperty()))
	{
		if (StructProperty->Struct == TBaseStructure<FColor>::Get())
		{
			return SNew(SFloatingPropertiesColorPropertyEditor, InPropertyHandle);
		}
	}

	return nullptr;
}

void SFloatingPropertiesColorPropertyEditor::Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	SFloatingPropertiesColorPropertyEditorBase::Construct(
		SFloatingPropertiesColorPropertyEditorBase::FArguments(),
		InPropertyHandle
	);
}

FLinearColor SFloatingPropertiesColorPropertyEditor::GetColorValue(FProperty* InProperty, UObject* InObject) const
{
	FColor Value;
	InProperty->GetValue_InContainer(InObject, &Value);
	return Value.ReinterpretAsLinear();
}

void SFloatingPropertiesColorPropertyEditor::SetColorValue(FProperty* InProperty, UObject* InObject, const FLinearColor& InNewValue)
{
	FColor Value = InNewValue.ToFColor(false);
	InProperty->SetValue_InContainer(InObject, &Value);
}
