// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/CustomPropertyWidgets/SFloatingPropertiesColorPropertyEditorBase.h"
#include "Engine/Engine.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"

#define LOCTEXT_NAMESPACE "SFloatingPropertiesPropertyWidget"

void SFloatingPropertiesColorPropertyEditorBase::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

void SFloatingPropertiesColorPropertyEditorBase::Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	PropertyHandleWeak = InPropertyHandle;
	bColorPickerOpen = false;

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(45.f)
		.HeightOverride(15.f)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(SColorBlock)
			.Color(this, &SFloatingPropertiesColorPropertyEditorBase::GetColorValue)
			.Size(FVector2D(45.f, 15.f))
			.ShowBackgroundForAlpha(true)
			.OnMouseButtonDown(this, &SFloatingPropertiesColorPropertyEditorBase::OnClickColorBlock)
		]
	];
}

FLinearColor SFloatingPropertiesColorPropertyEditorBase::GetColorValue() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();
	FLinearColor Value = FLinearColor::Black;

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			TArray<UObject*> Outers;
			PropertyHandle->GetOuterObjects(Outers);

			if (!Outers.IsEmpty() && IsValid(Outers[0]))
			{
				Value = GetColorValue(Property, Outers[0]);
			}
		}
	}

	return Value;	
}

void SFloatingPropertiesColorPropertyEditorBase::SetColorValue(const FLinearColor& InNewValue)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
	{
		return;
	}

	FProperty* Property = PropertyHandle->GetProperty();

	if (!Property)
	{
		return;
	}

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	for (UObject* Object : Objects)
	{
		if (IsValid(Object))
		{
			SetColorValue(Property, Object, InNewValue);
		}
	}
}

void SFloatingPropertiesColorPropertyEditorBase::RestoreOriginalColors()
{
	if (OriginalValues.IsEmpty())
	{
		return;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
	{
		return;
	}

	FProperty* Property = PropertyHandle->GetProperty();

	if (!Property)
	{
		return;
	}

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	for (UObject* Object : Objects)
	{
		if (IsValid(Object))
		{
			if (const FLinearColor* OriginalValue = OriginalValues.Find(Object))
			{
				SetColorValue(Property, Object, *OriginalValue);
			}
		}
	}

	OriginalValues.Empty();
}

FReply SFloatingPropertiesColorPropertyEditorBase::OnClickColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bColorPickerOpen)
	{
		return FReply::Unhandled();
	}

	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	// Needed for gamma
	if (!GEngine)
	{
		return FReply::Unhandled();
	}

	OriginalValues.Empty();

	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();
	TArray<UObject*> Objects;
	bool bValidProperty = false;

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			PropertyHandle->GetOuterObjects(Objects);

			if (!Objects.IsEmpty())
			{
				bValidProperty = true;
				FLinearColor Value;
				OriginalValues.Reserve(Objects.Num());

				for (UObject* Object : Objects)
				{
					if (IsValid(Object))
					{
						Value = GetColorValue(Property, Object);
						OriginalValues.Add(Object, Value);
					}
				}
			}
		}
	}

	if (!bValidProperty)
	{
		return FReply::Unhandled();
	}

	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bOnlyRefreshOnOk = true;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SFloatingPropertiesColorPropertyEditorBase::OnColorPickedCommitted);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SFloatingPropertiesColorPropertyEditorBase::OnColorPickerCancelled);
		PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SFloatingPropertiesColorPropertyEditorBase::OnColorPickerClosed);
		PickerArgs.bUseAlpha = true;
		PickerArgs.InitialColor = GetColorValue();
		PickerArgs.bOnlyRefreshOnMouseUp = false;
		PickerArgs.bOnlyRefreshOnOk = false;
	}

	bColorPickerOpen = OpenColorPicker(PickerArgs);

	if (bColorPickerOpen)
	{
		Transaction = MakeShared<FScopedTransaction>(LOCTEXT("Transaction", "Set color value"));

		for (UObject* Object : Objects)
		{
			Object->Modify();
		}
	}

	return FReply::Handled();
}

void SFloatingPropertiesColorPropertyEditorBase::OnColorPickedCommitted(FLinearColor InNewColor)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		PropertyHandle->NotifyPreChange();
	}

	SetColorValue(InNewColor);

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		PropertyHandle->NotifyPostChange(EPropertyChangeType::Interactive);
	}
}

void SFloatingPropertiesColorPropertyEditorBase::OnColorPickerCancelled(FLinearColor InOldColor)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		PropertyHandle->NotifyPreChange();
	}

	RestoreOriginalColors();

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		PropertyHandle->NotifyPostChange(EPropertyChangeType::Interactive);
	}
}

void SFloatingPropertiesColorPropertyEditorBase::OnColorPickerClosed(const TSharedRef<SWindow>& InClosedWindow)
{
	Transaction.Reset();
	OriginalValues.Empty();
	bColorPickerOpen = false;

	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		PropertyHandle->NotifyFinishedChangingProperties();
	}
}

#undef LOCTEXT_NAMESPACE
