// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Containers/Map.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyHandle;
class FScopedTransaction;
class SFloatingPropertiesPropertyWidget;
class UObject;

class SFloatingPropertiesColorPropertyEditorBase : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET(SFloatingPropertiesColorPropertyEditorBase, SCompoundWidget)

	SLATE_BEGIN_ARGS(SFloatingPropertiesColorPropertyEditorBase) {}
	SLATE_END_ARGS()

	virtual ~SFloatingPropertiesColorPropertyEditorBase() override = default;

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle);

protected:
	TWeakPtr<IPropertyHandle> PropertyHandleWeak;
	TSharedPtr<FScopedTransaction> Transaction;
	TMap<TWeakObjectPtr<UObject>, FLinearColor> OriginalValues;
	bool bColorPickerOpen;

	virtual FLinearColor GetColorValue(FProperty* InProperty, UObject* InObject) const = 0;

	virtual void SetColorValue(FProperty* InProperty, UObject* InObject, const FLinearColor& InNewValue) = 0;

	virtual FLinearColor GetColorValue() const;

	void SetColorValue(const FLinearColor& InNewValue);

	void RestoreOriginalColors();

	FReply OnClickColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void OnColorPickedCommitted(FLinearColor InNewColor);

	void OnColorPickerCancelled(FLinearColor InOldColor);

	void OnColorPickerClosed(const TSharedRef<SWindow>& InClosedWindow);
};
