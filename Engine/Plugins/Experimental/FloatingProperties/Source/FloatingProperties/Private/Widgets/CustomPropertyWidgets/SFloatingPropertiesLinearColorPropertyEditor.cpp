// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/CustomPropertyWidgets/SFloatingPropertiesLinearColorPropertyEditor.h"
#include "PropertyHandle.h"

void SFloatingPropertiesLinearColorPropertyEditor::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

TSharedPtr<SWidget> SFloatingPropertiesLinearColorPropertyEditor::CreateWidget(TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle->GetProperty()))
	{
		if (StructProperty->Struct == TBaseStructure<FColor>::Get())
		{
			return SNew(SFloatingPropertiesLinearColorPropertyEditor, InPropertyHandle);
		}
	}

	return nullptr;
}

void SFloatingPropertiesLinearColorPropertyEditor::Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	SFloatingPropertiesColorPropertyEditorBase::Construct(
		SFloatingPropertiesColorPropertyEditorBase::FArguments(),
		InPropertyHandle
	);
}

FLinearColor SFloatingPropertiesLinearColorPropertyEditor::GetColorValue(FProperty* InProperty, UObject* InObject) const
{
	FLinearColor Value;
	InProperty->GetValue_InContainer(InObject, &Value);
	return Value;
}

void SFloatingPropertiesLinearColorPropertyEditor::SetColorValue(FProperty* InProperty, UObject* InObject, const FLinearColor& InNewValue)
{
	InProperty->SetValue_InContainer(InObject, &InNewValue);
}
