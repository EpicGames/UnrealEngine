// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/CustomPropertyWidgets/SFloatingPropertiesColorPropertyEditorBase.h"

class SFloatingPropertiesLinearColorPropertyEditor : public SFloatingPropertiesColorPropertyEditorBase
{
public:
	SLATE_DECLARE_WIDGET(SFloatingPropertiesLinearColorPropertyEditor, SFloatingPropertiesColorPropertyEditorBase)

	SLATE_BEGIN_ARGS(SFloatingPropertiesLinearColorPropertyEditor) {}
	SLATE_END_ARGS()

	static TSharedPtr<SWidget> CreateWidget(TSharedRef<IPropertyHandle> InPropertyHandle);

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle);

protected:
	//~ Begin SFloatingPropertiesColorPropertyEditorBase
	virtual FLinearColor GetColorValue(FProperty* InProperty, UObject* InObject) const override;
	virtual void SetColorValue(FProperty* InProperty, UObject* InObject, const FLinearColor& InNewValue) override;
	//~ End SFloatingPropertiesColorPropertyEditorBase
};
