// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/FloatingPropertiesDragOperation.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SFloatingPropertiesPropertyWidget.h"
#include "Widgets/SFloatingPropertiesViewportWidget.h"

#define LOCTEXT_NAMESPACE "FloatingPropertiesDragOperation"

FFloatingPropertiesDragOperation::FFloatingPropertiesDragOperation(TSharedPtr<SFloatingPropertiesPropertyWidget> InPropertyWidget)
{
	PropertyWidgetWeak = InPropertyWidget;
	MouseStartPosition = FSlateApplication::Get().GetCursorPos();
	MouseToWidgetDelta = MouseStartPosition - InPropertyWidget->GetTickSpaceGeometry().GetAbsolutePosition();
}

FFloatingPropertiesDragOperation::~FFloatingPropertiesDragOperation()
{
	TSharedPtr<SFloatingPropertiesPropertyWidget> PropertyWidget = PropertyWidgetWeak.Pin();

	if (!PropertyWidget.IsValid())
	{
		return;
	}

	TSharedPtr<SFloatingPropertiesViewportWidget> ViewportWidget = PropertyWidget->GetViewportWidget();

	if (!ViewportWidget.IsValid())
	{
		return;
	}

	ViewportWidget->OnPropertyDragComplete(PropertyWidget.ToSharedRef());
}

void FFloatingPropertiesDragOperation::OnDragged(const FDragDropEvent& DragDropEvent)
{
	FDragDropOperation::OnDragged(DragDropEvent);

	TSharedPtr<SFloatingPropertiesPropertyWidget> PropertyWidget = PropertyWidgetWeak.Pin();

	if (!PropertyWidget.IsValid())
	{
		return;
	}

	TSharedPtr<SFloatingPropertiesViewportWidget> ViewportWidget = PropertyWidget->GetViewportWidget();

	if (!ViewportWidget.IsValid())
	{
		return;
	}

	ViewportWidget->OnPropertyDragUpdate(PropertyWidget.ToSharedRef(), MouseToWidgetDelta, MouseStartPosition, 
		FSlateApplication::Get().GetCursorPos());
}

#undef LOCTEXT_NAMESPACE
