// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class SFloatingPropertiesPropertyWidget;

class FFloatingPropertiesDragOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FFloatingPropertiesDragOperation, FDragDropOperation)

	FFloatingPropertiesDragOperation(TSharedPtr<SFloatingPropertiesPropertyWidget> InPropertyWidget);
	virtual ~FFloatingPropertiesDragOperation() override;

	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;

protected:
	TWeakPtr<SFloatingPropertiesPropertyWidget> PropertyWidgetWeak;
	FVector2f MouseStartPosition;
	FVector2f MouseToWidgetDelta;
};
