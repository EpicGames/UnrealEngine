// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFloatingPropertiesDragButton.h"
#include "FloatingPropertiesDragOperation.h"
#include "Styling/AppStyle.h"
#include "Widgets/SFloatingPropertiesPropertyWidget.h"

#define LOCTEXT_NAMESPACE "SFloatingPropertiesDragButton"

void SFloatingPropertiesDragButton::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

void SFloatingPropertiesDragButton::Construct(const FArguments& InArgs, TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget)
{
	PropertyWidgetWeak = InPropertyWidget;

	SButton::Construct(
		SButton::FArguments()
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(InArgs._OnClicked)
			[
				InArgs._Content.Widget
			]
	);
}

FReply SFloatingPropertiesDragButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SButton::OnMouseButtonDown(MyGeometry, MouseEvent);

	if (PropertyWidgetWeak.IsValid())
	{
		return Reply.DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return Reply;
}

FReply SFloatingPropertiesDragButton::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<SFloatingPropertiesPropertyWidget> PropertyWidget = PropertyWidgetWeak.Pin())
	{
		return FReply::Handled().BeginDragDrop(MakeShared<FFloatingPropertiesDragOperation>(PropertyWidget.ToSharedRef()));
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
