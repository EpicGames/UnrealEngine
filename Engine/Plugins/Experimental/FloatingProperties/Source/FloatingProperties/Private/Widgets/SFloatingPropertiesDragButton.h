// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SButton.h"

class SFloatingPropertiesPropertyWidget;

class SFloatingPropertiesDragButton : public SButton
{
public:
	SLATE_DECLARE_WIDGET(SFloatingPropertiesDragButton, SButton)

	SLATE_BEGIN_ARGS(SFloatingPropertiesDragButton)
		{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	virtual ~SFloatingPropertiesDragButton() override = default;

	void Construct(const FArguments& InArgs, TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget);

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget

protected:
	TWeakPtr<SFloatingPropertiesPropertyWidget> PropertyWidgetWeak;
};
