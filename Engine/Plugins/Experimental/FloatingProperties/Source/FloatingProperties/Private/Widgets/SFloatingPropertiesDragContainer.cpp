// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFloatingPropertiesDragContainer.h"
#include "Widgets/FloatingPropertiesDragOperation.h"

#define LOCTEXT_NAMESPACE "SFloatingPropertiesDragContainer"

void SFloatingPropertiesDragContainer::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

void SFloatingPropertiesDragContainer::Construct(const FArguments& InArgs, TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget)
{
	PropertyWidgetWeak = InPropertyWidget;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

FReply SFloatingPropertiesDragContainer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (PropertyWidgetWeak.IsValid())
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return FReply::Unhandled();
}

FReply SFloatingPropertiesDragContainer::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<SFloatingPropertiesPropertyWidget> PropertyWidget = PropertyWidgetWeak.Pin())
	{
		return FReply::Handled().BeginDragDrop(MakeShared<FFloatingPropertiesDragOperation>(PropertyWidget.ToSharedRef()));
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
