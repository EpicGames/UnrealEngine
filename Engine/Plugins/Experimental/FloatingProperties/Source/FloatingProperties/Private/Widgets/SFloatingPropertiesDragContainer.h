// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SFloatingPropertiesPropertyWidget;

class SFloatingPropertiesDragContainer : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET(SFloatingPropertiesDragContainer, SCompoundWidget)

	SLATE_BEGIN_ARGS(SFloatingPropertiesDragContainer)
		{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	virtual ~SFloatingPropertiesDragContainer() override = default;

	void Construct(const FArguments& InArgs, TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget);

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget

protected:
	TWeakPtr<SFloatingPropertiesPropertyWidget> PropertyWidgetWeak;
};
