// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFloatingPropertiesPropertyWidget.h"
#include "Algo/Reverse.h"
#include "ClassIconFinder.h"
#include "Components/ActorComponent.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "FloatingPropertiesModule.h"
#include "FloatingPropertiesSettings.h"
#include "IDetailTreeNode.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SFloatingPropertiesDragButton.h"
#include "Widgets/SFloatingPropertiesDragContainer.h"
#include "Widgets/SFloatingPropertiesViewportWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SFloatingPropertiesPropertyWidget"

void SFloatingPropertiesPropertyWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

void SFloatingPropertiesPropertyWidget::Construct(const FArguments& InArgs, TSharedRef<SFloatingPropertiesViewportWidget> InViewportWidget,
	const FFloatingPropertiesPropertyTypes& InPropertyTypes)
{
	ViewportWidgetWeak = InViewportWidget;
	PropertyHandleWeak = InPropertyTypes.PropertyHandle;

	SetVisibility(EVisibility::Visible);

	check(InPropertyTypes.PropertyHandle.IsValid());
	check(InPropertyTypes.Property);
	check(InPropertyTypes.TreeNode.IsValid());

	// Use first selected object to get property icons
	TArray<UObject*> OuterObjects;
	InPropertyTypes.PropertyHandle->GetOuterObjects(OuterObjects);
	check(!OuterObjects.IsEmpty());

	bool bAllowSaveValue = !InPropertyTypes.Property->IsA<FBoolProperty>();

	const FSlateBrush* Icon = nullptr;

	if (AActor* Actor = Cast<AActor>(OuterObjects[0]))
	{
		Icon = FClassIconFinder::FindIconForActor(Actor);
	}
	else if (UActorComponent* Component = Cast<UActorComponent>(OuterObjects[0]))
	{
		Icon = FSlateIconFinder::FindIconBrushForClass(Component->GetClass(), TEXT("SCS.NativeComponent"));
	}
	else
	{
		checkNoEntry();
	}

	{
		TSharedPtr<IPropertyHandle> LoopHandle = InPropertyTypes.PropertyHandle;
		UClass* OwningClass = nullptr;
		FString PropertyPath = "";
		TArray<FText> PropertyDisplayNames;

		do
		{
			if (FProperty* Property = LoopHandle->GetProperty())
			{
				PropertyPath = Property->GetName() + (PropertyPath.Len() > 0 ? FString(TEXT(".")) : FString()) + PropertyPath;
				PropertyDisplayNames.Add(Property->GetDisplayNameText());
				OwningClass = Property->GetOwnerClass();
			}

			LoopHandle = LoopHandle->GetParentHandle();
		}
		while (!OwningClass && LoopHandle.IsValid());

		ClassProperty = {TSoftClassPtr<UObject>(OwningClass),	PropertyPath};

		const FSlateFontInfo DetailFont = IDetailLayoutBuilder::GetDetailFont();

		Algo::Reverse(PropertyDisplayNames);

		PropertyNameWidget = SNew(STextBlock)
			.Font(DetailFont)
			.Text(FText::Join(INVTEXT("."), PropertyDisplayNames));
	}

	if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyTypes.Property))
	{
		const FFloatingPropertiesModule::FCreateStructPropertyValueWidgetDelegate* Delegate
			= FFloatingPropertiesModule::Get().GetStructPropertyValueWidgetDelegate(StructProperty->Struct);

		if (Delegate && Delegate->IsBound())
		{
			if (TSharedPtr<SWidget> Widget = Delegate->Execute(InPropertyTypes.PropertyHandle.ToSharedRef()))
			{
				PropertyValueWidget = Widget;
			}
		}

		if (!PropertyValueWidget.IsValid())
		{
			if (TSharedPtr<class IDetailPropertyRow> TreeRow = InPropertyTypes.TreeNode->GetRow())
			{
				TSharedPtr<SWidget> RowNameWidget;
				TSharedPtr<SWidget> RowValueWidget;
				FDetailWidgetRow Row;

				TreeRow->GetDefaultWidgets(RowNameWidget, RowValueWidget, Row, /* bAddWidgetDecoration */ false);

				if (RowValueWidget.IsValid())
				{
					PropertyValueWidget = RowValueWidget.ToSharedRef();
					bAllowSaveValue = false;
				}				
			}
		}
	}

	if (!PropertyValueWidget.IsValid())
	{
		PropertyValueWidget = InPropertyTypes.PropertyHandle->CreatePropertyValueWidget(false);
	}

	if (!PropertyValueWidget.IsValid())
	{
		PropertyValueWidget = SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> IconWidget = SNew(SImage)
		.DesiredSizeOverride(FVector2D(12.f, 12.f))
		.Image(Icon);

	if (!bAllowSaveValue)
	{
		IconWidget = SNew(SFloatingPropertiesDragContainer, SharedThis(this))
			[
				IconWidget
			];
	}
	else
	{
		IconWidget = SNew(SFloatingPropertiesDragButton, SharedThis(this))
			.OnClicked(this, &SFloatingPropertiesPropertyWidget::OnPropertyButtonClicked)
			[
				SAssignNew(PropertyAnchor, SMenuAnchor)
					.OnGetMenuContent(this, &SFloatingPropertiesPropertyWidget::MakePropertyMenu)
					[
						IconWidget
					]
			];
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(1.f, 1.f, 1.f, 1.f)
		.BorderImage(FAppStyle::GetBrush("ToolTip.Background"))
		.BorderBackgroundColor(FLinearColor(1.f, 1.f, 1.f, 0.75f))
		.OnMouseButtonDown_Lambda([](const FGeometry&, const FPointerEvent&) { return FReply::Handled(); })
		.OnMouseButtonUp_Lambda([](const FGeometry&, const FPointerEvent&) { return FReply::Handled(); })
		.OnMouseDoubleClick_Lambda([](const FGeometry&, const FPointerEvent&) { return FReply::Handled(); })
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 0.f, 0.f)
			[
				SNew(SBox)
				.WidthOverride(24.f)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					IconWidget
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3.f, 1.f, 0.f, 0.f)
			[
				SNew(SBox)
				.WidthOverride(this, &SFloatingPropertiesPropertyWidget::GetPropertyNameOverrideSize)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(SFloatingPropertiesDragContainer, SharedThis(this))
					[
						PropertyNameWidget.ToSharedRef()
					]
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3.f, 1.f, 0.f, 0.f)
			[
				SNew(SBox)
				.WidthOverride(this, &SFloatingPropertiesPropertyWidget::GetPropertyValueOverrideSize)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					PropertyValueWidget.ToSharedRef()
				]
			]
		]
	];
}

float SFloatingPropertiesPropertyWidget::GetPropertyNameWidgetWidth() const
{
	if (PropertyNameWidget.IsValid())
	{
		return PropertyNameWidget->GetDesiredSize().X;
	}

	return -1.f;
}

float SFloatingPropertiesPropertyWidget::GetPropertyValueWidgetWidth() const
{
	if (PropertyValueWidget.IsValid())
	{
		return PropertyValueWidget->GetDesiredSize().X;
	}

	return -1.f;
}

FFloatingPropertiesClassProperties* SFloatingPropertiesPropertyWidget::GetSavedValues(bool bInEnsure) const
{
	if (bInEnsure)
	{
		return &GetMutableDefault<UFloatingPropertiesSettings>()->SavedValues.FindOrAdd(ClassProperty);
	}

	if (UFloatingPropertiesSettings* FloatingPropertiesSettings = GetMutableDefault<UFloatingPropertiesSettings>())
	{
		if (FFloatingPropertiesClassProperties* ClassProperties = FloatingPropertiesSettings->SavedValues.Find(ClassProperty))
		{
			return ClassProperties;
		}
	}

	return nullptr;
}

FVector2f SFloatingPropertiesPropertyWidget::GetAbsolutePropertySize() const
{
	const FGeometry& NodePropertyWidgetGeometry = GetTickSpaceGeometry();
	const FVector2D LocalSize = NodePropertyWidgetGeometry.GetLocalSize();

	if (LocalSize.IsNearlyZero())
	{
		return FVector2f::ZeroVector;
	}

	return GetDesiredSize() * NodePropertyWidgetGeometry.GetAbsoluteSize() / LocalSize;
}

void SFloatingPropertiesPropertyWidget::SaveConfig()
{
	if (UFloatingPropertiesSettings* FloatingPropertiesSettings = GetMutableDefault<UFloatingPropertiesSettings>())
	{
		FloatingPropertiesSettings->SaveConfig();
	}
}

void SFloatingPropertiesPropertyWidget::AddSavedValue()
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FString NewName = "Value";

	if (const FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ false))
	{
		FString Value;
		PropertyHandle->GetValueAsFormattedString(Value);

		if (SavedValues->Properties.FindKey(Value))
		{
			FMessageDialog::Open(
				EAppMsgCategory::Error,
				EAppMsgType::Ok,
				LOCTEXT("AlreadySavedErrorBody", "This value has already been saved."),
				LOCTEXT("AlreadySavedErrorTitle", "Already Saved")
			);

			return;
		}

		NewName = UFloatingPropertiesSettings::FindUniqueName(*SavedValues, NewName);
	}

	TSharedRef<SEditableTextBox> NameInput = SNew(SEditableTextBox)
		.HintText(LOCTEXT("NewValueHint", "Saved Value Name"))
		.MinDesiredWidth(100.f)
		.Text(FText::FromString(NewName));

	SGenericDialogWidget::OpenDialog(
		LOCTEXT("NewValueTitle", "New Value Name"),
		NameInput,
		SGenericDialogWidget::FArguments(),
		/* Modal */ true
	);

	const FString NewValueName = NameInput->GetText().ToString().TrimStartAndEnd();

	if (NewValueName.IsEmpty())
	{
		UE_LOGF(LogFloatingProperties, Error, "Attempted to save a value with no name.");
		return;
	}

	AddSavedValue(NewValueName);
}

void SFloatingPropertiesPropertyWidget::AddSavedValue(const FString& InValueName)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FProperty* Property = PropertyHandle->GetProperty();

	if (!Property)
	{
		return;
	}

	FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ true);

	if (!SavedValues)
	{
		return;
	}

	const FString ValueName = UFloatingPropertiesSettings::FindUniqueName(*SavedValues, InValueName);

	FString Value;
	PropertyHandle->GetValueAsFormattedString(Value);

	SavedValues->Properties.Add(ValueName, Value);
	SaveConfig();
}

void SFloatingPropertiesPropertyWidget::ApplySavedValue(FString InValueName)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	const FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ false);

	if (!SavedValues)
	{
		return;
	}

	const FString* SavedValuePtr = SavedValues->Properties.Find(InValueName);

	if (!SavedValuePtr)
	{
		return;
	}

	PropertyHandle->SetValueFromFormattedString(*SavedValuePtr);
}

bool SFloatingPropertiesPropertyWidget::IsValueActive(FString InValueName) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	const FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ false);

	if (!SavedValues)
	{
		return false;
	}

	const FString* ValuePtr = SavedValues->Properties.Find(InValueName);

	if (!ValuePtr)
	{
		return false;
	}

	FString Value;
	PropertyHandle->GetValueAsFormattedString(Value);

	return ValuePtr->Equals(Value);
}

void SFloatingPropertiesPropertyWidget::RemoveAllValues()
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ false);

	if (!SavedValues)
	{
		return;
	}

	SavedValues->Properties.Empty();
	SaveConfig();
}

bool SFloatingPropertiesPropertyWidget::CanSaveValue(FString InValue) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	const FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ false);

	if (!SavedValues)
	{
		return true;
	}

	return !SavedValues->Properties.FindKey(InValue);
}

FReply SFloatingPropertiesPropertyWidget::OnPropertyButtonClicked()
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!PropertyAnchor.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!PropertyAnchor->IsOpen())
	{
		PropertyAnchor->SetIsOpen(/* bInIsOpen */ true, /* bFocusMenu */ true);
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SFloatingPropertiesPropertyWidget::MakePropertyMenu()
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();

	if (!PropertyHandle.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FProperty* Property = PropertyHandle->GetProperty();

	if (!Property)
	{
		return SNullWidget::NullWidget;
	}

	if (Property->IsA<FBoolProperty>())
	{
		return SNullWidget::NullWidget;
	}

	FString Value;
	PropertyHandle->GetValueAsFormattedString(Value);

	FUIAction ResetToDefaultAction
	(
		FExecuteAction::CreateSP(PropertyHandle.Get(), &IPropertyHandle::ResetToDefault),
		FCanExecuteAction::CreateSP(PropertyHandle.Get(), &IPropertyHandle::CanResetToDefault)
	);

	FUIAction AddSavedValueAction
	(
		FExecuteAction::CreateSP(this, &SFloatingPropertiesPropertyWidget::AddSavedValue),
		FCanExecuteAction::CreateSP(this, &SFloatingPropertiesPropertyWidget::CanSaveValue, Value)
	);

	FMenuBuilder MenuBuilder(/* bInShouldCloseWindowAfterMenuSelection */ true, nullptr, nullptr);

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("PropertyResetToDefaultLabel", "Reset To Default"),
		LOCTEXT("PropertyResetToDefaultToolTip", "Reset to the property to its default value."),
		FSlateIcon(),
		ResetToDefaultAction,
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("PropertySaveValueLabel", "Save Value"),
		LOCTEXT("PropertySaveValueToolTip", "Save the current value."),
		FSlateIcon(),
		AddSavedValueAction,
		NAME_None,
		EUserInterfaceActionType::Button
	);

	const FFloatingPropertiesClassProperties* SavedValues = GetSavedValues(/* bInEnsure */ false);

	if (SavedValues && !SavedValues->Properties.IsEmpty())
	{
		MenuBuilder.AddMenuSeparator();

		for (const TPair<FString, FString>& Pair : SavedValues->Properties)
		{
			FUIAction SetSavedValue
			(
				FExecuteAction::CreateSP(this, &SFloatingPropertiesPropertyWidget::ApplySavedValue, Pair.Key),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SFloatingPropertiesPropertyWidget::IsValueActive, Pair.Key)
			);

			MenuBuilder.AddMenuEntry
			(
				FText::FromString(Pair.Key),
				FText::Format(LOCTEXT("ValueTooltip", "Value: {0}"), FText::FromString(Pair.Value)),
				FSlateIcon(),
				SetSavedValue,
				NAME_None,
				EUserInterfaceActionType::Check
			);
		}

		MenuBuilder.AddMenuSeparator();

		FUIAction RemoveAllValues
		(
			FExecuteAction::CreateSP(this, &SFloatingPropertiesPropertyWidget::RemoveAllValues)
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("PropertyRemoveAllValuesLabel", "Remove All Values"),
			LOCTEXT("PropertyRemoveAllValuesTooltio", "Remove all the saved values."),
			FSlateIcon(),
			RemoveAllValues,
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
