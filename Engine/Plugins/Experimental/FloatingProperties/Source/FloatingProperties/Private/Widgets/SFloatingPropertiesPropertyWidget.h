// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "FloatingPropertiesSettings.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateStructs.h"

class IPropertyHandle;
class SFloatingPropertiesViewportWidget;
class SMenuAnchor;
class STextBlock;
struct FFloatingPropertiesClassProperties;
struct FFloatingPropertiesPropertyTypes;

class SFloatingPropertiesPropertyWidget : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET(SFloatingPropertiesPropertyWidget, SCompoundWidget)

	SLATE_BEGIN_ARGS(SFloatingPropertiesPropertyWidget) {}
	SLATE_END_ARGS()

	virtual ~SFloatingPropertiesPropertyWidget() override = default;

	void Construct(const FArguments& InArgs, TSharedRef<SFloatingPropertiesViewportWidget> InViewportWidget,
		const FFloatingPropertiesPropertyTypes& InPropertyTypes);

	TSharedPtr<SFloatingPropertiesViewportWidget> GetViewportWidget() const { return ViewportWidgetWeak.Pin(); }

	TSharedPtr<IPropertyHandle> GetPropertyHandle() const { return PropertyHandleWeak.Pin(); }

	const FFloatingPropertiesClassProperty& GetClassProperty() const { return ClassProperty; }

	float GetPropertyNameWidgetWidth() const;

	float GetPropertyValueWidgetWidth() const;

	void SetPropertyNameOverrideSize(FOptionalSize InSize) { PropertyNameOverrideSize = InSize; }

	void SetPropertyValueOverrideSize(FOptionalSize InSize) { PropertyValueOverrideSize = InSize; }

	FVector2f GetAbsolutePropertySize() const;

	void SaveConfig();

protected:
	TWeakPtr<SFloatingPropertiesViewportWidget> ViewportWidgetWeak;

	TWeakPtr<IPropertyHandle> PropertyHandleWeak;

	TSharedPtr<SMenuAnchor> PropertyAnchor;

	TSharedPtr<SWidget> PropertyNameWidget;

	TSharedPtr<SWidget> PropertyValueWidget;

	FFloatingPropertiesClassProperty ClassProperty;

	FOptionalSize PropertyNameOverrideSize;

	FOptionalSize PropertyValueOverrideSize;

	FFloatingPropertiesClassProperties* GetSavedValues(bool bInEnsure) const;

	FOptionalSize GetPropertyNameOverrideSize() const { return PropertyNameOverrideSize; }

	FOptionalSize GetPropertyValueOverrideSize() const { return PropertyValueOverrideSize; }

	void AddSavedValue();

	void AddSavedValue(const FString& InValueName);

	void ApplySavedValue(FString InValueName);

	bool IsValueActive(FString InValueName) const;

	bool CanSaveValue(FString InValue) const;

	void RemoveAllValues();

	FReply OnPropertyButtonClicked();

	TSharedRef<SWidget> MakePropertyMenu();
};