// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SFloatingPropertiesViewportWidget.h"
#include "Components/ActorComponent.h"
#include "Data/FloatingPropertiesPropertyNodeContainer.h"
#include "Data/FloatingPropertiesPropertyNode.h"
#include "Data/FloatingPropertiesSnapMetrics.h"
#include "Data/FloatingPropertiesWidgetController.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "PropertyHandle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCanvas.h"
#include "Widgets/SFloatingPropertiesPropertyWidget.h"

#define LOCTEXT_NAMESPACE "SFloatingPropertiesViewportWidget"

namespace UE::FloatingProperties::Private
{
	constexpr float MinPropertyValueWidth = 100.f;

	void HomogenizeStackWidth(TSharedRef<FFloatingPropertiesPropertyNode> InNode)
	{
		TArray<TSharedRef<FFloatingPropertiesPropertyNode>> StackNodes = InNode->GetNodeStack();

		float MaxNameWidth = -1.f;

		float MaxValueWidth = -1.f;

		for (const TSharedRef<FFloatingPropertiesPropertyNode>& StackNode : StackNodes)
		{
			TSharedRef<SFloatingPropertiesPropertyWidget> AttachedWidget = StackNode->GetPropertyWidget();
			bool bAllowNarrowWidget = false;

			if (TSharedPtr<IPropertyHandle> PropertyHandle = AttachedWidget->GetPropertyHandle())
			{
				if (FProperty* Property = PropertyHandle->GetProperty())
				{
					/**
					 * Bools only get a really tiny edit widget which does not require a good width to display properly.
					 * Other types, such as text or numeric types, when their edit widgets attempt to use their preferred
					 * size, display really tiny, unusable widgets. Thus, for non-bool properties a minimum width is
					 * specified so that they display nicely.
					 */
					if (Property->IsA<FBoolProperty>())
					{
						bAllowNarrowWidget = true;
					}

					// Struct properties are responsible for managing their own width.
					else if (Property->IsA<FStructProperty>())
					{
						bAllowNarrowWidget = true;
					}
				}
			}

			const float NameWidth = AttachedWidget->GetPropertyNameWidgetWidth();
			float ValueWidth = AttachedWidget->GetPropertyValueWidgetWidth();

			if (!bAllowNarrowWidget)
			{
				ValueWidth = FMath::Max(ValueWidth, MinPropertyValueWidth);
			}			

			MaxNameWidth = FMath::Max(MaxNameWidth, NameWidth);
			MaxValueWidth = FMath::Max(MaxValueWidth, ValueWidth);
		}

		for (const TSharedRef<FFloatingPropertiesPropertyNode>& StackNode : StackNodes)
		{
			if (MaxNameWidth > 0.f)
			{
				StackNode->GetPropertyWidget()->SetPropertyNameOverrideSize(FOptionalSize(MaxNameWidth));
			}
			else
			{
				StackNode->GetPropertyWidget()->SetPropertyNameOverrideSize(FOptionalSize());
			}

			if (MaxValueWidth > 0.f)
			{
				StackNode->GetPropertyWidget()->SetPropertyValueOverrideSize(FOptionalSize(MaxValueWidth));
			}
			else
			{
				StackNode->GetPropertyWidget()->SetPropertyValueOverrideSize(FOptionalSize());
			}
		}
	}
}

void SFloatingPropertiesViewportWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& InInitializer)
{
}

void SFloatingPropertiesViewportWidget::Construct(const FArguments& InArgs)
{
	SetVisibility(EVisibility::SelfHitTestInvisible);

	Properties = MakeShared<FloatingPropertiesPropertyNodeContainer>(SharedThis(this));

	ChildSlot
	[
		SNew(SBox)
		.Padding(2.f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.Visibility_Lambda([IsVisibleCallback = InArgs._IsVisible]()
			{
				const bool bIsVisible = IsVisibleCallback.IsBound() ? IsVisibleCallback.Execute() : true;

				return bIsVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
			})
		[
			SAssignNew(FieldContainer, SCanvas)
			.Visibility(EVisibility::SelfHitTestInvisible)
		]
	];

	Reset();
}

void SFloatingPropertiesViewportWidget::BuildWidget(AActor* InSelectedActor, const FFloatingPropertiesPropertyList& InActorProperties,
	UActorComponent* InSelectedComponent, const FFloatingPropertiesPropertyList& InComponentProperties)
{
	Reset();

	const bool bValidActor = IsValid(InSelectedActor);
	const bool bValidComponent = IsValid(InSelectedComponent);

	if (!bValidActor && !bValidComponent)
	{
		return;
	}

	// Get property list for this selection
	TArray<FFloatingPropertiesPropertyTypes> PropertyList;

	if (bValidActor)
	{
		PropertyList.Append(InActorProperties.Properties);
	}

	if (bValidComponent)
	{
		PropertyList.Append(InComponentProperties.Properties);
	}
	
	Properties->Reserve(PropertyList.Num());

	TMap<FFloatingPropertiesClassProperty, TSharedRef<FFloatingPropertiesPropertyNode>> ClassPropertyToNode;
	ClassPropertyToNode.Reserve(PropertyList.Num());

	// Build class property list/widgets
	for (const FFloatingPropertiesPropertyTypes& Property : PropertyList)
	{
		if (!Property.Property || !Property.PropertyHandle.IsValid() || Property.PropertyHandle->GetNumOuterObjects() == 0)
		{
			continue;
		}

		TSharedRef<SFloatingPropertiesPropertyWidget> NewWidget =
			SNew(SFloatingPropertiesPropertyWidget, SharedThis(this), Property);

		TSharedRef<FFloatingPropertiesPropertyNode> Node = Properties->AddWidget(NewWidget);

		FFloatingPropertiesClassProperty ClassProperty = Node->GetPropertyWidget()->GetClassProperty();

		ClassPropertyToNode.Add(ClassProperty, Node);

		FieldContainer->AddSlot()
			.Size(this, &SFloatingPropertiesViewportWidget::GetNodeSize, Node->GetPropertyWidget().ToWeakPtr())
			.Position(this, &SFloatingPropertiesViewportWidget::GetNodePosition, Node.ToWeakPtr())
			[
				Node->GetPropertyWidget()
			];
	}

	// Load data
	const UFloatingPropertiesSettings* FloatingPropertiesSettings = GetDefault<UFloatingPropertiesSettings>();

	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : Properties->GetNodes())
	{
		FFloatingPropertiesClassProperty ClassProperty = Node->GetPropertyWidget()->GetClassProperty();

		if (const FFloatingPropertiesClassPropertyPosition* SavedPropertyPosition = FloatingPropertiesSettings->PropertyPositions.Find(ClassProperty))
		{
			Node->SetPropertyPositionDirect(*SavedPropertyPosition);
		}

		if (const FFloatingPropertiesClassPropertyAnchor* SavedPropertyAnchor = FloatingPropertiesSettings->PropertyAnchors.Find(ClassProperty))
		{
			if (SavedPropertyAnchor->ParentProperty.IsValid())
			{
				if (const TSharedRef<FFloatingPropertiesPropertyNode>* ParentNode = ClassPropertyToNode.Find(SavedPropertyAnchor->ParentProperty))
				{
					if (!Node->HasSibling(*ParentNode))
					{
						Node->SetParent((*ParentNode)->GetStackLeafMostNode());
					}
				}

				if (const TSharedRef<FFloatingPropertiesPropertyNode>* ChildNode = ClassPropertyToNode.Find(SavedPropertyAnchor->ChildProperty))
				{
					if (!Node->HasSibling(*ChildNode))
					{
						Node->SetChild((*ChildNode)->GetStackRootNode());
					}
				}
			}
		}
	}

	// Build the default stack at the bottom left
	TArray<TSharedRef<FFloatingPropertiesPropertyNode>> UnpositionedProperties;
	UnpositionedProperties.Reserve(Properties->GetNodes().Num());

	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : Properties->GetNodes())
	{
		if (!Node->HasParent() && Node->GetPropertyPosition().OnDefaultStack())
		{
			UnpositionedProperties.Add(Node);
		}
	}

	if (UnpositionedProperties.Num() > 1)
	{
		TSharedPtr<FFloatingPropertiesPropertyNode> PreviousNode;

		for (int32 Index = 0; Index < UnpositionedProperties.Num(); ++Index)
		{
			TSharedRef<FFloatingPropertiesPropertyNode> Node = UnpositionedProperties[Index];

			if (PreviousNode.IsValid())
			{
				Node->SetParent(PreviousNode.ToSharedRef());
			}

			PreviousNode = Node;
		}
	}

	// Nodes cannot be laid out here because we still don't know our size. Hysteresis powers combine!
}

TSharedRef<SWidget> SFloatingPropertiesViewportWidget::AsWidget()
{
	return SharedThis(this);
}

FVector2f SFloatingPropertiesViewportWidget::GetDraggableArea() const
{
	static const FVector2f BorderArea = FVector2f(4.f, 4.f);

	const FGeometry& Geometry = GetTickSpaceGeometry();

	return Geometry.GetAbsoluteSize() - BorderArea;
}

void SFloatingPropertiesViewportWidget::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	SCompoundWidget::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

	if (!FMath::IsNearlyZero(GetTickSpaceGeometry().GetLocalSize().X))
	{
		CheckForViewportSizeChange(AllottedGeometry.GetLocalSize());

		if (EnsurePositions(/* bInInvalidate */ !bInitialPositionsSet))
		{
			bInitialPositionsSet = true;
		}
	}
}

void SFloatingPropertiesViewportWidget::OnPropertyDragUpdate(TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget, 
	const FVector2f& InMouseToWidgetDelta, const FVector2f& InMouseStartPosition, const FVector2f& InMouseCurrentPosition)
{
	TSharedPtr<FFloatingPropertiesPropertyNode> DraggedNodePtr = Properties->FindNodeForWidget(InPropertyWidget);

	if (!DraggedNodePtr.IsValid())
	{
		return;
	}

	TSharedRef<FFloatingPropertiesPropertyNode> DraggedNode = DraggedNodePtr.ToSharedRef();

	const FVector2f DraggableArea = GetDraggableArea();
	constexpr float MinAxisValue = UE_SMALL_NUMBER;

	if (DraggableArea.X < MinAxisValue || DraggableArea.Y < MinAxisValue)
	{
		return;
	}

	using namespace UE::FloatingProperties::Private;

	// Work out the space we have to play with
	const FVector2f DragAreaPosition = GetTickSpaceGeometry().GetAbsolutePosition();
	const FVector2f DraggedToMousePosition = InMouseCurrentPosition - InMouseToWidgetDelta;
	const FVector2f DraggedNodeStackSize = DraggedNode->GetStackSize();

	FFloatingPropertiesSnapMetrics DraggedNodeMetrics[2] = {
		{
			DraggedNode,
			DraggedToMousePosition,
			0.f,
			EFloatingPropertiesSnapType::AttachAsChild
		},
		{
			DraggedNode,
			DraggedToMousePosition + FVector2f(0.f, DraggedNodeStackSize.Y),
			0.f,
			EFloatingPropertiesSnapType::AttachAsParent
		}
	};

	// If we're breaking away from the stack
	const bool bIsolateProperty = FSlateApplication::Get().GetModifierKeys().IsAltDown();

	if (bIsolateProperty)
	{
		DraggedNode->RemoveFromStack();
		DraggedNode->SetPropertyPosition(DraggableArea, DraggedNodeMetrics[FFloatingPropertiesSnapMetrics::TopLeft].Position - DragAreaPosition);
		HomogenizeStackWidth(DraggedNode);
		return;
	}

	TSharedPtr<FFloatingPropertiesPropertyNode> CurrentParent = DraggedNode->GetParent();

	// Check to see if we're still locked to our current stack
	if (CurrentParent.IsValid())
	{
		const FVector2f OriginalParentSnapPoint = FFloatingPropertiesSnapMetrics::GetSnapPosition(
			CurrentParent.ToSharedRef(), 
			EFloatingPropertiesSnapType::AttachAsParent
		);

		const float DistanceFromParentSq = (DraggedNodeMetrics[FFloatingPropertiesSnapMetrics::TopLeft].Position - OriginalParentSnapPoint).SizeSquared();

		if (DistanceFromParentSq < FFloatingPropertiesSnapMetrics::SnapBreakDistanceSq)
		{
			return;
		}
	}

	TSet<TSharedRef<FFloatingPropertiesPropertyNode>> DraggedStackSet(DraggedNode->GetNodeStack());

	TArray<FFloatingPropertiesSnapMetrics> SnapMetrics;
	SnapMetrics.Reserve((Properties->GetNodes().Num() - 1) * 2);

	// Gather a list of snap candidates.
	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : Properties->GetNodes())
	{
		// We're only interested nodes outside of the dragged stack.
		if (DraggedStackSet.Contains(Node))
		{
			continue;
		}

		// Match the bottom left of the node stack to the top left of the dragged node
		if (!Node->HasChild())
		{
			SnapMetrics.Add(FFloatingPropertiesSnapMetrics::Make(
				Node, 
				DraggedNodeMetrics[FFloatingPropertiesSnapMetrics::TopLeft].Position, 
				EFloatingPropertiesSnapType::AttachAsParent
			));
		}

		// Match the top left of the node stack to the bottom left of the dragged node
		if (!Node->HasParent())
		{
			SnapMetrics.Add(FFloatingPropertiesSnapMetrics::Make(
				Node, 
				DraggedNodeMetrics[FFloatingPropertiesSnapMetrics::BottomLeft].Position, 
				EFloatingPropertiesSnapType::AttachAsChild
			));
		}
	}

	// Work out the closest
	int32 ClosestMetricsIndex = INDEX_NONE;

	for (int32 Index = 0; Index < SnapMetrics.Num(); ++Index)
	{
		const FFloatingPropertiesSnapMetrics& SnapMetric = SnapMetrics[Index];

		if (SnapMetric.DistanceSq < FFloatingPropertiesSnapMetrics::SnapDistanceSq &&
			(ClosestMetricsIndex == INDEX_NONE || SnapMetric.DistanceSq < SnapMetrics[ClosestMetricsIndex].DistanceSq))
		{
			ClosestMetricsIndex = Index;
		}
	}

	// Work out if we need to change node relationships
	TSharedPtr<FFloatingPropertiesPropertyNode> NewParent;
	TSharedPtr<FFloatingPropertiesPropertyNode> NewChild;
	bool bNodeRelationshipChanged = false;

	if (ClosestMetricsIndex != INDEX_NONE)
	{
		FFloatingPropertiesSnapMetrics& SnapMetric = SnapMetrics[ClosestMetricsIndex];

		switch (SnapMetric.AttachType)
		{
			case EFloatingPropertiesSnapType::AttachAsParent:
				NewParent = SnapMetric.Node;
				break;

			case EFloatingPropertiesSnapType::AttachAsChild:
				NewChild = SnapMetric.Node;
				break;
		}
	}

	// Attach new parent, if any change.
	if (CurrentParent != NewParent)
	{
		bNodeRelationshipChanged = true;

		if (CurrentParent.IsValid())
		{
			// Enter free drag
			DraggedNode->RemoveParent();

			// Update the old parent's stack width
			HomogenizeStackWidth(CurrentParent.ToSharedRef());
		}

		if (NewParent.IsValid())
		{
			// Make sure we don't have a parent
			if (TSharedPtr<FFloatingPropertiesPropertyNode> NewParentOldChild = NewParent->RemoveChild())
			{
				HomogenizeStackWidth(NewParentOldChild.ToSharedRef());
			}

			// Snap to parent
			DraggedNode->SetParent(NewParent.ToSharedRef());
		}
	}

	// Attach new child, if there is one.
	if (NewChild.IsValid())
	{
		bNodeRelationshipChanged = true;

		// Make sure we don't have a child
		if (TSharedPtr<FFloatingPropertiesPropertyNode> NewChildOldParent = NewChild->RemoveParent())
		{
			HomogenizeStackWidth(NewChildOldParent.ToSharedRef());
		}

		DraggedNode->GetStackLeafMostNode()->SetChild(NewChild.ToSharedRef());
	}

	// If we now don't have a parent, update the position for dragging.
	if (!DraggedNode->HasParent())
	{
		// Snap to sides of the screen
		const FVector2f AvailableDraggableArea = DraggableArea - DraggedNode->GetStackSize();
		FVector2f DraggedToPosition = DraggedNodeMetrics[FFloatingPropertiesSnapMetrics::TopLeft].Position - DragAreaPosition;

		FFloatingPropertiesSnapMetrics::SnapToDraggableArea(AvailableDraggableArea, DraggedToPosition);

		DraggedNode->SetPropertyPosition(DraggableArea, DraggedToPosition);
	}

	// If we changed node attachments anywhere, make sure we're all the same width.
	if (bNodeRelationshipChanged)
	{
		HomogenizeStackWidth(DraggedNode);
	}
}

void SFloatingPropertiesViewportWidget::OnPropertyDragComplete(TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget)
{
	InPropertyWidget->SaveConfig();
}

void SFloatingPropertiesViewportWidget::Reset()
{
	bInitialPositionsSet = false;
	Properties->Empty();
	FieldContainer->ClearChildren();
}

void SFloatingPropertiesViewportWidget::CheckForViewportSizeChange(const FVector2f& InViewportSize) const
{
	const FIntPoint CurrentRenderSize = {
		FMath::RoundToInt(InViewportSize.X),
		FMath::RoundToInt(InViewportSize.Y)
	};

	if (CurrentRenderSize != LastRenderSize)
	{
		Properties->InvalidateAllPositions();
		LastRenderSize = CurrentRenderSize;
	}
}

bool SFloatingPropertiesViewportWidget::EnsurePositions(bool bInInvalidate) const
{
	using namespace UE::FloatingProperties::Private;

	for (const TSharedRef<FFloatingPropertiesPropertyNode>& Node : Properties->GetNodes())
	{
		if (!Node->HasParent())
		{
			switch (Node->UpdateStackPositions(bInInvalidate))
			{
				case EFloatingPropertiesUpdateResult::Failed:
					return false;

				case EFloatingPropertiesUpdateResult::AlreadyUpToDate:
					break;

				case EFloatingPropertiesUpdateResult::Updated:
					HomogenizeStackWidth(Node);
					break;
			}
		}
	}

	return true;
}

FVector2D SFloatingPropertiesViewportWidget::GetNodeSize(TWeakPtr<SFloatingPropertiesPropertyWidget> InNodeWidgetWeak) const
{
	if (TSharedPtr<SFloatingPropertiesPropertyWidget> Widget = InNodeWidgetWeak.Pin())
	{
		return Widget->GetDesiredSize();
	}

	return FVector2D::ZeroVector;
}

FVector2D SFloatingPropertiesViewportWidget::GetNodePosition(TWeakPtr<FFloatingPropertiesPropertyNode> InNodeWeak) const
{
	if (TSharedPtr<FFloatingPropertiesPropertyNode> Node = InNodeWeak.Pin())
	{
		const FGeometry& TickSpaceGeometry = GetTickSpaceGeometry();
		const FVector2f LocalSize = TickSpaceGeometry.GetAbsoluteSize();

		if (LocalSize.IsNearlyZero())
		{
			return FVector2D::ZeroVector;
		}

		return static_cast<FVector2D>(Node->GetCachedPosition() * TickSpaceGeometry.GetLocalSize() / LocalSize);
	}

	return FVector2D::ZeroVector;
}

#undef LOCTEXT_NAMESPACE
