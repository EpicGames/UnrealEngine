// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Containers/Array.h"
#include "Data/FloatingPropertiesPropertyNodeContainer.h"
#include "Data/FloatingPropertiesWidgetController.h"
#include "FloatingPropertiesSettings.h"
#include "Templates/SharedPointer.h"

class AActor;
class IPropertyHandle;
class SCanvas;
class SFloatingPropertiesPropertyWidget;
class UActorComponent;
struct FFloatingPropertiesPropertyList;

class SFloatingPropertiesViewportWidget : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET(SFloatingPropertiesViewportWidget, SCompoundWidget)

	SLATE_BEGIN_ARGS(SFloatingPropertiesViewportWidget)
		{}
		SLATE_EVENT(FIsVisibleDelegate, IsVisible)
	SLATE_END_ARGS()

	virtual ~SFloatingPropertiesViewportWidget() override = default;

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget
	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	//~ End SWidget

	void BuildWidget(AActor* InSelectedActor, const FFloatingPropertiesPropertyList& InActorProperties,
		UActorComponent* InSelectedComponent, const FFloatingPropertiesPropertyList& InComponentProperties);

	TSharedRef<SWidget> AsWidget();

	FVector2f GetDraggableArea() const;

	/**
	 * The bottom of the stack attaches to the top of the closest snap node if it's above the node.
	 *
	 * The top of the stack attaches to the bottom of the closest snap node if it's below the node.
	 *
	 * Attaching to the bottom of another stack is temporary. If the node continues to be dragged, after
	 * SnapBreakDistanceSq from the snapped-to node, it will break off the stack it attached to and
	 * continue to be dragged.
	 */
	void OnPropertyDragUpdate(TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget, const FVector2f& InMouseToWidgetDelta,
		const FVector2f& InMouseStartPosition, const FVector2f& InMouseCurrentPosition);

	void OnPropertyDragComplete(TSharedRef<SFloatingPropertiesPropertyWidget> InPropertyWidget);

protected:
	TSharedPtr<SCanvas> FieldContainer;

	TSharedPtr<FloatingPropertiesPropertyNodeContainer> Properties;

	mutable FIntPoint LastRenderSize;

	mutable bool bInitialPositionsSet;

	void Reset();

	void CheckForViewportSizeChange(const FVector2f& InViewportSize) const;

	bool EnsurePositions(bool bInInvalidate) const;

	FVector2D GetNodeSize(TWeakPtr<SFloatingPropertiesPropertyWidget> InNodeWidgetWeak) const;

	FVector2D GetNodePosition(TWeakPtr<FFloatingPropertiesPropertyNode> InNodeWeak) const;
};
