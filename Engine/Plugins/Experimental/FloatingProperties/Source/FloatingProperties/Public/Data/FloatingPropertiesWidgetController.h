// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Data/IFloatingPropertiesDataProvider.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "GameFramework/Actor.h"
#include "Templates/SharedPointer.h"

class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class SFloatingPropertiesViewportWidget;
class SFloatingPropertiesViewportWidget;
class SWidget;
class USelection;
class UTypedElementSelectionSet;

struct FFloatingPropertiesPropertyTypes
{
	TSharedPtr<IPropertyHandle> PropertyHandle;
	FProperty* Property;
	TSharedPtr<IDetailTreeNode> TreeNode;

	~FFloatingPropertiesPropertyTypes()
	{
		PropertyHandle.Reset();
		TreeNode.Reset();
	}
};

struct FFloatingPropertiesPropertyList
{
	TSharedPtr<IPropertyRowGenerator> Generator;
	TArray<FFloatingPropertiesPropertyTypes> Properties;

	~FFloatingPropertiesPropertyList()
	{
		Properties.Empty();
		Generator.Reset();
	}
};

DECLARE_DELEGATE_RetVal(bool, FIsVisibleDelegate);

class FLOATINGPROPERTIES_API FFloatingPropertiesWidgetController : public TSharedFromThis<FFloatingPropertiesWidgetController>
{
public:
	FFloatingPropertiesWidgetController(TSharedRef<IFloatingPropertiesDataProvider> InDataProvider);

	virtual ~FFloatingPropertiesWidgetController();

	virtual void Init();

protected:
	TSharedPtr<IFloatingPropertiesDataProvider> DataProvider;

	TWeakObjectPtr<AActor> SelectedActorWeak;

	TWeakObjectPtr<UActorComponent> SelectedComponentWeak;

	FFloatingPropertiesPropertyList ActorProperties;

	FFloatingPropertiesPropertyList ComponentProperties;

	TSharedRef<SFloatingPropertiesViewportWidget> CreateWidget(FIsVisibleDelegate InVisibilityCallback);

	void UpdatePropertiesForObject(UObject* InObject, FFloatingPropertiesPropertyList& OutPropertyList);

	void RebuildWidgets(const FFloatingPropertiesPropertyList& InActorProperties, const FFloatingPropertiesPropertyList& InComponentProperties);

	void RemoveWidgets();

	void OnViewportChange();

	void RegisterSelectionChanged();

	void UnregisterSelectionChanged();

	void OnSelectionChange(const UTypedElementSelectionSet* InSelectionSet);

	void OnSelectionChange();

	void OnSelectionChange(UWorld* InWorld, USelection* InActorSelection, USelection* InComponentSelection);
};
