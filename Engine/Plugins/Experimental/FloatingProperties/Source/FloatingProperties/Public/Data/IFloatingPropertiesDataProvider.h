// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointerFwd.h"

class FOnSelectionChanged;
class IFloatingPropertiesWidgetContainer;
class USelection;
class UWorld;

class IFloatingPropertiesDataProvider
{
public:
	/** Must provide consistent shared refs. Same container, same ref. */
	virtual TArray<TSharedRef<IFloatingPropertiesWidgetContainer>> GetWidgetContainers() = 0;

	virtual USelection* GetActorSelection() const = 0;

	virtual USelection* GetComponentSelection() const = 0;

	virtual UWorld* GetWorld() const = 0;

	virtual bool IsWidgetVisibleInContainer(TSharedRef<IFloatingPropertiesWidgetContainer> InContainer) const = 0;
};
