// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SFloatingPropertiesViewportWidget;

class IFloatingPropertiesWidgetContainer
{
public:
	/** Returns the added widget. */
	virtual TSharedPtr<SFloatingPropertiesViewportWidget> GetWidget() const { return nullptr; }

	/** Adds the widget to teh container, which should store it for later. */
	virtual bool SetWidget(TSharedRef<SFloatingPropertiesViewportWidget> InWidget) = 0;

	/** Removes the added widget. */
	virtual bool RemoveWidget() = 0;

	/** Returns true if the widget's container is still valid. */
	virtual bool IsValid() const = 0;
};
