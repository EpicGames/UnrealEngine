// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

class FFloatingPropertiesLevelEditorWidgetController;
class ILevelEditor;
class IPropertyHandle;
class SWidget;
class UFloatingPropertiesSettings;
class UScriptStruct;

DECLARE_LOG_CATEGORY_EXTERN(LogFloatingProperties, Log, All);

/**
 * Floating Properties - Adds floating details panel properties on to the viewport.
 */
class FFloatingPropertiesModule : public IModuleInterface
{
public:
	static FFloatingPropertiesModule& Get();

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FCreateStructPropertyValueWidgetDelegate, TSharedRef<IPropertyHandle>);

	void RegiserStructPropertyValueWidgetDelegate(UScriptStruct* InStruct, FCreateStructPropertyValueWidgetDelegate InDelegate);

	const FCreateStructPropertyValueWidgetDelegate* GetStructPropertyValueWidgetDelegate(UScriptStruct* InStruct) const;

	void UnregiserStructPropertyValueWidgetDelegate(UScriptStruct* InStruct);

protected:
	TSharedPtr<FFloatingPropertiesLevelEditorWidgetController> LevelEditorWidgetController;
	TMap<FName, FCreateStructPropertyValueWidgetDelegate> PropertyValueWidgetDelegates;

	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor);

	void CreateLevelEditorWidgetController(TSharedPtr<ILevelEditor> InLevelEditor);

	void DestroyLevelEditorWidgetController();

	void OnSettingsChanged(const UFloatingPropertiesSettings* InSettings, FName InSetting);

	void AddDefaultStructPropertyValueWidgetDelegates();

	void OnEnginePreExit();
};
