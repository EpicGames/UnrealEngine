// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Math/IntPoint.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "FloatingPropertiesSettings.generated.h"

enum EHorizontalAlignment : int;
enum EVerticalAlignment : int;

USTRUCT(BlueprintType)
struct FFloatingPropertiesClassProperty
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	TSoftClassPtr<UObject> Class;

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	FString PropertyPath;

	bool operator==(const FFloatingPropertiesClassProperty& InOther) const
	{
		return Class == InOther.Class && PropertyPath.Equals(InOther.PropertyPath);
	}

	bool IsValid() const
	{
		return Class.IsValid() && !PropertyPath.IsEmpty();
	}

	void Reset()
	{
		Class = nullptr;
		PropertyPath = "";
	}

	friend uint32 GetTypeHash(const FFloatingPropertiesClassProperty& InClassProperty)
	{
		return HashCombineFast(GetTypeHash(InClassProperty.Class), GetTypeHash(InClassProperty.PropertyPath));
	}
};

USTRUCT(BlueprintType)
struct FFloatingPropertiesClassProperties
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	TMap<FString, FString> Properties;
};

/** The Fill alignments are used as a "not set" value. */
USTRUCT(BlueprintType)
struct FFloatingPropertiesClassPropertyPosition
{
	GENERATED_BODY()

	static const FFloatingPropertiesClassPropertyPosition DefaultStackPosition;

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	TEnumAsByte<EHorizontalAlignment> HorizontalAnchor = EHorizontalAlignment::HAlign_Fill;

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	TEnumAsByte<EVerticalAlignment> VerticalAnchor = EVerticalAlignment::VAlign_Fill;

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	FIntPoint Offset = FIntPoint::ZeroValue;

	bool OnDefaultStack() const;
};

USTRUCT(BlueprintType)
struct FFloatingPropertiesClassPropertyAnchor
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	FFloatingPropertiesClassProperty ParentProperty;

	UPROPERTY(EditAnywhere, Category = "Floating Properties")
	FFloatingPropertiesClassProperty ChildProperty;
};

UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "Floating Properties"))
class FLOATINGPROPERTIES_API UFloatingPropertiesSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFloatingPropertiesSettingsChanged, const UFloatingPropertiesSettings* /* This */, FName /* Setting name */)
	static FOnFloatingPropertiesSettingsChanged OnChange;

	static FString FindUniqueName(const FFloatingPropertiesClassProperties& InPropertyList, const FString& InDefault);

	UFloatingPropertiesSettings();

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Floating Properties")
	bool bEnabled;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Floating Properties")
	TMap<FFloatingPropertiesClassProperty, FFloatingPropertiesClassProperties> SavedValues;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Floating Properties")
	TMap<FFloatingPropertiesClassProperty, FFloatingPropertiesClassPropertyPosition> PropertyPositions;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Floating Properties")
	TMap<FFloatingPropertiesClassProperty, FFloatingPropertiesClassPropertyAnchor> PropertyAnchors;

	//~ Begin UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject
};
